// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraSubtitles_WinGDK : ElectraSubtitles
	{
		public ElectraSubtitles_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
