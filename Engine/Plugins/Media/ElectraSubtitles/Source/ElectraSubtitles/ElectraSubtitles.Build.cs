// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraSubtitles : ModuleRules
	{
		public ElectraSubtitles(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"ElectraBase",
					"Expat"
				});
			// Expat is a static library, not a DLL.
			PrivateDefinitions.Add("XML_STATIC=1");
		}
	}
}
