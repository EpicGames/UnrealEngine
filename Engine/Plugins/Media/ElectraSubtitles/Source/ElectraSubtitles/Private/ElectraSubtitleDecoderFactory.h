// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IElectraSubtitleDecoder;

class IElectraSubtitleDecoderFactory
{
public:
	virtual ~IElectraSubtitleDecoderFactory() = default;
	virtual TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> CreateDecoder(const FString& SubtitleCodecName) = 0;
};


class IElectraSubtitleDecoderFactoryRegistry
{
public:
	virtual ~IElectraSubtitleDecoderFactoryRegistry() = default;

	struct FCodecInfo
	{
		FString	CodecName;
		int32 Priority = 0;
	};

	virtual void AddDecoderFactory(const TArray<FCodecInfo>& InCodecInformation, IElectraSubtitleDecoderFactory* InDecoderFactory) = 0;

};

