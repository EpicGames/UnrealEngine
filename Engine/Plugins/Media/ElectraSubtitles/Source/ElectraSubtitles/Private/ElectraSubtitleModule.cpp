// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraSubtitleModule.h"


#include "Features/IModularFeatures.h"
#include "IElectraSubtitleModule.h"
#include "ElectraSubtitleDecoderFactory.h"

#include "Modules/ModuleManager.h"
#include "tx3g/ElectraSubtitleDecoder_TX3G.h"
#include "wvtt/ElectraSubtitleDecoder_WVTT.h"
#include "ttml/ElectraSubtitleDecoder_TTML.h"

#define LOCTEXT_NAMESPACE "ElectraSubtitlesModule"

DEFINE_LOG_CATEGORY(LogElectraSubtitles);

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraSubtitleModularFeature : public IElectraSubtitleModularFeature, public IElectraSubtitleDecoderFactoryRegistry
{
public:
	virtual ~FElectraSubtitleModularFeature();

	//-------------------------------------------------------------------------
	// Methods from IElectraSubtitleModularFeature
	//
	virtual bool SupportsFormat(const FString& SubtitleCodecName) const override;

	virtual void GetSupportedFormats(TArray<FString>& OutSupportedCodecNames) const override;

	virtual int32 GetPriorityForFormat(const FString& SubtitleCodecName) const override;

	virtual TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> CreateDecoderForFormat(const FString& SubtitleCodecName) override;

	//-------------------------------------------------------------------------
	// Methods from IElectraSubtitleDecoderFactoryRegistry
	//
	virtual void AddDecoderFactory(const TArray<FCodecInfo>& InCodecInformation, IElectraSubtitleDecoderFactory* InDecoderFactory) override;

private:
	struct FSupportedCodec
	{
		int32 Priority = -1;
		IElectraSubtitleDecoderFactory* Factory = nullptr;

	};
	mutable FCriticalSection Lock;
	TMap<FString, FSupportedCodec> SupportedCodecs;
};

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraSubtitlesModule : public IElectraSubtitlesModule
{
public:
	virtual void StartupModule() override
	{
		// Register the codecs this plugin provides.
		FElectraSubtitleDecoderTX3G::RegisterCodecs(ModularFeature);
		FElectraSubtitleDecoderWVTT::RegisterCodecs(ModularFeature);
		FElectraSubtitleDecoderTTML::RegisterCodecs(ModularFeature);

		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), &ModularFeature);
	}

	virtual void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), &ModularFeature);
	}
	
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

private:
	FElectraSubtitleModularFeature ModularFeature;
};

IMPLEMENT_MODULE(FElectraSubtitlesModule, ElectraSubtitles);

// -----------------------------------------------------------------------------------------------------------------------------------

void FElectraSubtitleModularFeature::AddDecoderFactory(const TArray<FCodecInfo>& InCodecInformation, IElectraSubtitleDecoderFactory* InDecoderFactory)
{
	FScopeLock lock(&Lock);
	// Every codec may claim support for more than one format, so we add it in multiple times, once per format.
	// Also, if there is already a registered format we compare the priorities and take only the one with a higher priority.
	// In case of a tie in priority we take the last one added.
	for(auto &ci : InCodecInformation)
	{
		bool bAdd = true;
		for(auto &Codec : SupportedCodecs)
		{
			if (Codec.Key.Equals(ci.CodecName) && Codec.Value.Priority > ci.Priority)
			{
				bAdd = false;
				break;
			}
		}
		if (bAdd)
		{
			SupportedCodecs.Emplace(ci.CodecName, FSupportedCodec( {ci.Priority, InDecoderFactory} ));
		}
	}
}


FElectraSubtitleModularFeature::~FElectraSubtitleModularFeature()
{
}

bool FElectraSubtitleModularFeature::SupportsFormat(const FString& SubtitleCodecName) const
{
	FScopeLock lock(&Lock);
	return SupportedCodecs.Contains(SubtitleCodecName);
}

void FElectraSubtitleModularFeature::GetSupportedFormats(TArray<FString>& OutSupportedCodecNames) const
{
	FScopeLock lock(&Lock);
	SupportedCodecs.GenerateKeyArray(OutSupportedCodecNames);
}

int32 FElectraSubtitleModularFeature::GetPriorityForFormat(const FString& SubtitleCodecName) const
{
	FScopeLock lock(&Lock);
	const FSupportedCodec* Codec = SupportedCodecs.Find(SubtitleCodecName);
	return Codec ? Codec->Priority : -1;
}

TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> FElectraSubtitleModularFeature::CreateDecoderForFormat(const FString& SubtitleCodecName)
{
	FScopeLock lock(&Lock);
	const FSupportedCodec* Codec = SupportedCodecs.Find(SubtitleCodecName);
	return Codec ? Codec->Factory->CreateDecoder(SubtitleCodecName) : nullptr;
}


#undef LOCTEXT_NAMESPACE

