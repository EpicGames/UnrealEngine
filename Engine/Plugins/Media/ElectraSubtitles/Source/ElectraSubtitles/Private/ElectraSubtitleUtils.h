// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformString.h"

namespace ElectraSubtitleUtils
{

	inline bool StringEquals(const TCHAR * const s1, const TCHAR * const s2)
	{ 
		return FPlatformString::Strcmp(s1, s2) == 0; 
	}

	inline bool StringStartsWith(const TCHAR * const s1, const TCHAR * const s2, SIZE_T n)
	{ 
		return FPlatformString::Strncmp(s1, s2, n) == 0; 
	}



#if !PLATFORM_LITTLE_ENDIAN
	static inline uint8 GetFromBigEndian(uint8 value)		{ return value; }
	static inline int8 GetFromBigEndian(int8 value)			{ return value; }
	static inline uint16 GetFromBigEndian(uint16 value)		{ return value; }
	static inline int16 GetFromBigEndian(int16 value)		{ return value; }
	static inline int32 GetFromBigEndian(int32 value)		{ return value; }
	static inline uint32 GetFromBigEndian(uint32 value)		{ return value; }
	static inline int64 GetFromBigEndian(int64 value)		{ return value; }
	static inline uint64 GetFromBigEndian(uint64 value)		{ return value; }
#else
	static inline uint16 EndianSwap(uint16 value)			{ return (value >> 8) | (value << 8); }
	static inline int16 EndianSwap(int16 value)				{ return int16(EndianSwap(uint16(value))); }
	static inline uint32 EndianSwap(uint32 value)			{ return (value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24); }
	static inline int32 EndianSwap(int32 value)				{ return int32(EndianSwap(uint32(value))); }
	static inline uint64 EndianSwap(uint64 value)			{ return (uint64(EndianSwap(uint32(value & 0xffffffffU))) << 32) | uint64(EndianSwap(uint32(value >> 32))); }
	static inline int64 EndianSwap(int64 value)				{ return int64(EndianSwap(uint64(value)));}
	static inline uint8 GetFromBigEndian(uint8 value)		{ return value; }
	static inline int8 GetFromBigEndian(int8 value)			{ return value; }
	static inline uint16 GetFromBigEndian(uint16 value)		{ return EndianSwap(value); }
	static inline int16 GetFromBigEndian(int16 value)		{ return EndianSwap(value); }
	static inline int32 GetFromBigEndian(int32 value)		{ return EndianSwap(value); }
	static inline uint32 GetFromBigEndian(uint32 value)		{ return EndianSwap(value); }
	static inline int64 GetFromBigEndian(int64 value)		{ return EndianSwap(value); }
	static inline uint64 GetFromBigEndian(uint64 value)		{ return EndianSwap(value); }
#endif

	template <typename T>
	static inline T ValueFromBigEndian(const T value)
	{
		return GetFromBigEndian(value);
	}

}

