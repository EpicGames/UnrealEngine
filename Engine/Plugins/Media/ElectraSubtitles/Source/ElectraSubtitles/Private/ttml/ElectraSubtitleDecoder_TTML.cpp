// Copyright Epic Games, Inc. All Rights Reserved.

#include "ttml/ElectraSubtitleDecoder_TTML.h"
#include "ElectraSubtitleDecoderFactory.h"
#include "ttml/TTMLParser.h"
#include "ElectraSubtitleModule.h"
#include "ttml/TTMLSubtitleHandler.h"
#include "MediaDecoderOutput.h"


class FElectraSubtitleDecoderFactoryTTML : public IElectraSubtitleDecoderFactory
{
public:
	virtual TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> CreateDecoder(const FString& SubtitleCodecName) override;
};


void FElectraSubtitleDecoderTTML::RegisterCodecs(IElectraSubtitleDecoderFactoryRegistry& InRegistry)
{
	static FElectraSubtitleDecoderFactoryTTML Factory;
	TArray<IElectraSubtitleDecoderFactoryRegistry::FCodecInfo> CodecInfos
	{
		{ FString(TEXT("stpp")), 0},
		{ FString(TEXT("application/ttml+xml")), 0},
	// See: https://www.w3.org/TR/ttml-profile-registry/#registry-profile-designator-specifications
		{ FString(TEXT("stpp.ttml.im1t")), 0},
		{ FString(TEXT("stpp.ttml.im2t")), 0},
		{ FString(TEXT("stpp.ttml.im3t")), 0}
	};
	InRegistry.AddDecoderFactory(CodecInfos, &Factory);
}


TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> FElectraSubtitleDecoderFactoryTTML::CreateDecoder(const FString& SubtitleCodecName)
{
	return MakeShared<FElectraSubtitleDecoderTTML, ESPMode::ThreadSafe>(SubtitleCodecName);
}






class FSubtitleDecoderOutputTTML : public ISubtitleDecoderOutput
{
public:
	virtual ~FSubtitleDecoderOutputTTML() = default;

	void SetText(const FString& InText)
	{
		FTCHARToUTF8 Converted(*InText); // Convert to UTF8
		TextAsArray.Empty();
		TextAsArray.Append(reinterpret_cast<const uint8*>(Converted.Get()), Converted.Length());
	}

	void SetDuration(const Electra::FTimeValue& InDuration)
	{
		Duration = InDuration.GetAsTimespan();
	}

	void SetTimestamp(const Electra::FTimeValue& InTimestamp)
	{
		Timestamp.Time = InTimestamp.GetAsTimespan();
		Timestamp.SequenceIndex = InTimestamp.GetSequenceIndex();
	}

	void SetID(const FString& InID)
	{
		ID = InID;
	}


	const TArray<uint8>& GetData() override
	{
		return TextAsArray;
	}

	FDecoderTimeStamp GetTime() const override
	{
		return Timestamp;
	}

	void SetTime(FDecoderTimeStamp& InTime) override
	{
		Timestamp = InTime;
	}

	FTimespan GetDuration() const override
	{
		return Duration;
	}

	const FString& GetFormat() const override
	{
		static FString Format(TEXT("stpp"));
		return Format;
	}
	const FString& GetID() const override
	{
		return ID;
	}
private:
	TArray<uint8> TextAsArray;
	FString ID;
	FDecoderTimeStamp Timestamp;
	FTimespan Duration;
};



namespace ElectraSubtitleDecoderTTMLOptions
{
	static const FName PresentationTimeOffset(TEXT("PresentationTimeOffset"));
	static const FName SendEmptySubtitleDuringGaps(TEXT("sendEmptySubtitleDuringGaps"));
	static const FName SourceID(TEXT("source_id"));
	static const FName Width(TEXT("width"));
	static const FName Height(TEXT("height"));
	static const FName OffsetX(TEXT("offset_x"));
	static const FName OffsetY(TEXT("offset_y"));
	static const FName Timescale(TEXT("timescale"));
}




FElectraSubtitleDecoderTTML::FElectraSubtitleDecoderTTML(const FString& CodecOrMimetype)
{
	if (CodecOrMimetype.Equals(TEXT("application/ttml+xml")))
	{
		DocumentType = EDocumentType::TTMLXML;
	}
	else
	{
		DocumentType = EDocumentType::STPP;
	}
	DurationOfEmptySubtitle.SetFromSeconds(1.0);
	DurationWindow.SetFromSeconds(0.001);
}

FElectraSubtitleDecoderTTML::~FElectraSubtitleDecoderTTML()
{
}

bool FElectraSubtitleDecoderTTML::InitializeStreamWithCSD(const TArray<uint8>& InCSD, const Electra::FParamDict& InAdditionalInfo)
{
	// Get dimension, placement offset and other data from the sideband dictionary.
	Width = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::Width).SafeGetInt64(0);
	Height = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::Height).SafeGetInt64(0);
	TranslationX = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::OffsetX).SafeGetInt64(0);
	TranslationY = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::OffsetY).SafeGetInt64(0);
	Timescale = (uint32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::Timescale).SafeGetInt64(0);
	bSendEmptySubtitleDuringGaps = InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::SendEmptySubtitleDuringGaps).SafeGetBool();

	return true;
}

IElectraSubtitleDecoder::FOnSubtitleReceivedDelegate& FElectraSubtitleDecoderTTML::GetParsedSubtitleReceiveDelegate()
{
	return ParsedSubtitleDelegate;
}

Electra::FTimeValue FElectraSubtitleDecoderTTML::GetStreamedDeliveryTimeOffset()
{
	return Electra::FTimeValue::GetZero();
}

void FElectraSubtitleDecoderTTML::AddStreamedSubtitleData(const TArray<uint8>& InData, Electra::FTimeValue InAbsoluteTimestamp, Electra::FTimeValue InDuration, const Electra::FParamDict& InAdditionalInfo)
{
	ClearLastError();

	TSharedPtr<ElectraTTMLParser::ITTMLSubtitleHandler, ESPMode::ThreadSafe> SubtitleHandler;
	FString SourceID;
	if (DocumentType == EDocumentType::TTMLXML)
	{
		// There can be only a single document active.
		AccessLock.Lock();
		TimelineHandlers.Empty();
		AccessLock.Unlock();

		// For sideloaded data we try to get a unique ID we can use to check if we are still operating on the
		// same data. If we are we do not need to parse the entire document again.
		SourceID = InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::SourceID).SafeGetFString();
		if (SideloadedHandler.ContentID.Equals(SourceID) && SideloadedHandler.Handler.IsValid())
		{
			SubtitleHandler = SideloadedHandler.Handler;

			FScopeLock Lock(&AccessLock);
			FTimelineHandlers NewTimelineHandler;
			NewTimelineHandler.Handler = SubtitleHandler;
			NewTimelineHandler.AbsoluteTimeRange.Start = InAbsoluteTimestamp;
			NewTimelineHandler.AbsoluteTimeRange.End = InAbsoluteTimestamp + InDuration;
			NewTimelineHandler.MediaLocalPTO = InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::PresentationTimeOffset).SafeGetTimeValue(Electra::FTimeValue::GetZero());
			TimelineHandlers.Emplace(MoveTemp(NewTimelineHandler));
		}
	}
	if (!SubtitleHandler.IsValid())
	{
		TSharedPtr<ElectraTTMLParser::ITTMLParser, ESPMode::ThreadSafe> Parser = ElectraTTMLParser::ITTMLParser::Create();
		if (Parser->ParseXMLDocument(InData, Electra::FParamDict()))
		{
			Electra::FTimeValue Start = Electra::FTimeValue::GetZero();
			Electra::FTimeValue Duration = Electra::FTimeValue::GetPositiveInfinity();
			Electra::FParamDict Options;
			if (DocumentType == EDocumentType::TTMLXML)
			{
				// See option description for ITTMLParser::BuildSubtitleList
				Options.Set(ElectraSubtitleDecoderTTMLOptions::SendEmptySubtitleDuringGaps, Electra::FVariantValue(bSendEmptySubtitleDuringGaps));
			}
			else
			{
				// We do not ask the parser subtitle list to insert empty subtitles because with streamed subtitles
				// the parser does not have access to future subtitles to make that decision.
			}
			if (Parser->BuildSubtitleList(Start, Duration, Options))
			{
				SubtitleHandler = Parser->GetSubtitleHandler();
				// Remember this subtitle list for sideloaded data so we do not need to parse it over again after a seek.
				if (DocumentType == EDocumentType::TTMLXML)
				{
					SideloadedHandler.Handler = SubtitleHandler;
					SideloadedHandler.ContentID = SourceID;
				}

				FScopeLock Lock(&AccessLock);
				FTimelineHandlers NewTimelineHandler;
				NewTimelineHandler.Handler = SubtitleHandler;
				NewTimelineHandler.AbsoluteTimeRange.Start = InAbsoluteTimestamp;
				NewTimelineHandler.AbsoluteTimeRange.End = InAbsoluteTimestamp + InDuration;
				NewTimelineHandler.MediaLocalPTO = InAdditionalInfo.GetValue(ElectraSubtitleDecoderTTMLOptions::PresentationTimeOffset).SafeGetTimeValue(Electra::FTimeValue::GetZero());
				TimelineHandlers.Emplace(MoveTemp(NewTimelineHandler));
				TimelineHandlers.StableSort([](const FTimelineHandlers& e1, const FTimelineHandlers& e2)
				{
					return e1.AbsoluteTimeRange.Start < e2.AbsoluteTimeRange.Start;
				});
			}
			else
			{
				LastErrorMsg = Parser->GetLastErrorMessage();
				UE_LOGF(LogElectraSubtitles, Error, "TTML error: %ls", *LastErrorMsg);
			}
		}
		else
		{
			LastErrorMsg = Parser->GetLastErrorMessage();
			UE_LOGF(LogElectraSubtitles, Error, "%ls", *LastErrorMsg);
		}
	}
}

void FElectraSubtitleDecoderTTML::SignalStreamedSubtitleEOD()
{
}

void FElectraSubtitleDecoderTTML::Flush()
{
	if (SideloadedHandler.Handler.IsValid())
	{
		SideloadedHandler.Handler->ClearActiveRange();
	}

	FScopeLock Lock(&AccessLock);
	TimelineHandlers.Empty();
	LastPlaybackUpdateAbsPos.SetToInvalid();
}

void FElectraSubtitleDecoderTTML::Start()
{
}

void FElectraSubtitleDecoderTTML::Stop()
{
}

void FElectraSubtitleDecoderTTML::UpdatePlaybackPosition(Electra::FTimeValue InAbsolutePosition, Electra::FTimeValue InLocalPosition)
{
	TArray<FTimelineHandlers> ActiveTimelineHandlers;

	// Check if the sequence index has changed due to a seek or loop.
	// Typically we get a Flush() followed with another call here that is still prior to the loop point
	// which results in locking the current position to the end of the looping range since we are only
	// checking in the forward play direction.
	// When the loop actually happens we need to reset so we start at the beginning.
	if (!LastPlaybackUpdateAbsPos.IsValid() || LastPlaybackUpdateAbsPos.GetSequenceIndex() != InAbsolutePosition.GetSequenceIndex())
	{
		if (SideloadedHandler.Handler.IsValid())
		{
			SideloadedHandler.Handler->ClearActiveRange();
		}
	}
	LastPlaybackUpdateAbsPos = InAbsolutePosition;


	// When streaming each access unit contains (or is supposed to contain) a complete TTML document.
	// The start and end times in the document are not adjusted for the segment duration. This means
	// that we will also get the same subtitle more than once in every segment whose time overlaps
	// the display time of the subtitle.
	// Due to the sequential nature of the AUs the AUs cannot themselves overlap and only one can be
	// active at any moment in time. We maintain a time-sorted list of them anyway, but only the
	// last one containing the time we are currently at will be used.
	AccessLock.Lock();
	for(int32 i=0; i<TimelineHandlers.Num(); ++i)
	{
		if (InAbsolutePosition >= TimelineHandlers[i].AbsoluteTimeRange.End)
		{
			TimelineHandlers.RemoveAt(i--);
		}
		else if (TimelineHandlers[i].AbsoluteTimeRange.Contains(InAbsolutePosition))
		{
			ActiveTimelineHandlers.Emplace(TimelineHandlers[i]);
		}
	}
	AccessLock.Unlock();

	if (ActiveTimelineHandlers.Num())
	{
		const FTimelineHandlers& Active = ActiveTimelineHandlers.Last();
		Electra::FTimeRange Range;
		Range.Start = InLocalPosition + Active.MediaLocalPTO;
		Range.End = Range.Start + DurationWindow;
		if (Active.Handler->UpdateActiveRange(Range))
		{
			TArray<ElectraTTMLParser::ITTMLSubtitleHandler::FActiveSubtitle> Subtitles;
			Active.Handler->GetActiveSubtitles(Subtitles);
			if (Subtitles.Num())
			{
				FString Text;
				Electra::FTimeValue FirstActiveTime;
				Electra::FTimeValue LastActiveTime;
				// We need to combine the active subtitles into a single one.
				for(int32 i=0; i<Subtitles.Num(); ++i)
				{
					ElectraTTMLParser::ITTMLSubtitleHandler::FActiveSubtitle& Subtitle = Subtitles[i];
					if (i)
					{
						Text.AppendChar(TCHAR('\n'));
					}
					Text.Append(Subtitle.Text);
					if (!LastActiveTime.IsValid() || Subtitle.TimeRange.End < LastActiveTime)
					{
						LastActiveTime = Subtitle.TimeRange.End;
					}
					if (!FirstActiveTime.IsValid() || Subtitle.TimeRange.Start < FirstActiveTime)
					{
						FirstActiveTime = Subtitle.TimeRange.Start;
					}
				}
				// Map from local to absolute time.
				Electra::FTimeValue LocalToAbsOffset = InAbsolutePosition - InLocalPosition - Active.MediaLocalPTO;
				FirstActiveTime += LocalToAbsOffset;
				LastActiveTime += LocalToAbsOffset;
				if (FirstActiveTime < Active.AbsoluteTimeRange.Start)
				{
					FirstActiveTime = Active.AbsoluteTimeRange.Start;
				}
				if (LastActiveTime > Active.AbsoluteTimeRange.End)
				{
					LastActiveTime = Active.AbsoluteTimeRange.End;
				}
				FirstActiveTime.SetSequenceIndex(InAbsolutePosition.GetSequenceIndex());
				LastActiveTime.SetSequenceIndex(InAbsolutePosition.GetSequenceIndex());

				TSharedPtr<FSubtitleDecoderOutputTTML, ESPMode::ThreadSafe> Out = MakeShared<FSubtitleDecoderOutputTTML, ESPMode::ThreadSafe>();
				Out->SetText(Text);
				Out->SetTimestamp(FirstActiveTime);
				Electra::FTimeValue Duration = LastActiveTime - FirstActiveTime;
				if (LastActiveTime < Electra::FTimeValue::GetZero())
				{
					Duration.SetToZero();
				}
				Out->SetDuration(Duration);
				ParsedSubtitleDelegate.Broadcast(Out);
				//UE_LOGF(LogElectraSubtitles, Log, "[%.3f-%.3f] @%.3f: %.3f-%.3f: %ls", Active.AbsoluteTimeRange.Start.GetAsSeconds(), Active.AbsoluteTimeRange.End.GetAsSeconds(), InAbsolutePosition.GetAsSeconds(), FirstActiveTime.GetAsSeconds(), (FirstActiveTime+Duration).GetAsSeconds(), *Text);

				// Check if we need to send an empty subtitle.
				if (DocumentType == EDocumentType::STPP && bSendEmptySubtitleDuringGaps && Active.AbsoluteTimeRange.End > LastActiveTime)
				{
					Out = MakeShared<FSubtitleDecoderOutputTTML, ESPMode::ThreadSafe>();
					Out->SetTimestamp(LastActiveTime);
					Duration = Active.AbsoluteTimeRange.End - LastActiveTime;
					if (Duration > DurationOfEmptySubtitle)
					{
						Duration = DurationOfEmptySubtitle;
					}
					Out->SetDuration(Duration);
					ParsedSubtitleDelegate.Broadcast(Out);
					//UE_LOGF(LogElectraSubtitles, Log, "[%.3f-%.3f] @%.3f: %.3f-%.3f:", Active.AbsoluteTimeRange.Start.GetAsSeconds(), Active.AbsoluteTimeRange.End.GetAsSeconds(), InAbsolutePosition.GetAsSeconds(), LastActiveTime.GetAsSeconds(), (LastActiveTime+Duration).GetAsSeconds());
				}
			}
		}
	}
}


void FElectraSubtitleDecoderTTML::ClearLastError()
{
	LastErrorMsg.Empty();
}
