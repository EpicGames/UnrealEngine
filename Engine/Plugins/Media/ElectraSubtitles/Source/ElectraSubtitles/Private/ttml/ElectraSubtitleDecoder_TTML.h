// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraSubtitleDecoder.h"
#include "PlayerTime.h"

class IElectraSubtitleDecoderFactoryRegistry;
namespace ElectraTTMLParser { class ITTMLSubtitleHandler; }

/**
 * TTML / IMSC1 subtitle decoder (https://www.w3.org/TR/ttml2/ and https://www.w3.org/TR/ttml-imsc1.2/)
 * 
 * This is for the IMSC1 text profile only. Images are not supported.
 */
class FElectraSubtitleDecoderTTML : public IElectraSubtitleDecoder
{
public:
	static void RegisterCodecs(IElectraSubtitleDecoderFactoryRegistry& InRegistry);

	FElectraSubtitleDecoderTTML(const FString& CodecOrMimetype);

	virtual ~FElectraSubtitleDecoderTTML();

	//-------------------------------------------------------------------------
	// Methods from IElectraSubtitleDecoder
	//
	virtual bool InitializeStreamWithCSD(const TArray<uint8>& InCSD, const Electra::FParamDict& InAdditionalInfo) override;
	virtual IElectraSubtitleDecoder::FOnSubtitleReceivedDelegate& GetParsedSubtitleReceiveDelegate() override;
	virtual Electra::FTimeValue GetStreamedDeliveryTimeOffset() override;
	virtual void AddStreamedSubtitleData(const TArray<uint8>& InData, Electra::FTimeValue InAbsoluteTimestamp, Electra::FTimeValue InDuration, const Electra::FParamDict& InAdditionalInfo) override;
	virtual void SignalStreamedSubtitleEOD() override;
	virtual void Flush() override;
	virtual void Start() override;
	virtual void Stop() override;
	virtual void UpdatePlaybackPosition(Electra::FTimeValue InAbsolutePosition, Electra::FTimeValue InLocalPosition) override;

private:
	enum class EDocumentType
	{
		Undefined,
		STPP,			//!< ISO/IEC 14496-30
		TTMLXML			//!< sideloaded XML document
	};

	struct FSideloadedSubtitleHandler
	{
		FString ContentID;
		TSharedPtr<ElectraTTMLParser::ITTMLSubtitleHandler, ESPMode::ThreadSafe> Handler;
	};

	struct FTimelineHandlers
	{
		TSharedPtr<ElectraTTMLParser::ITTMLSubtitleHandler, ESPMode::ThreadSafe> Handler;
		Electra::FTimeRange AbsoluteTimeRange;
		Electra::FTimeValue MediaLocalPTO;
	};

	void ClearLastError();


	Electra::FTimeValue DurationOfEmptySubtitle;
	Electra::FTimeValue DurationWindow;
	bool bSendEmptySubtitleDuringGaps = false;

	FOnSubtitleReceivedDelegate ParsedSubtitleDelegate;
	EDocumentType DocumentType = EDocumentType::Undefined;
	FString LastErrorMsg;

	FCriticalSection AccessLock;
	TArray<FTimelineHandlers> TimelineHandlers;
	FSideloadedSubtitleHandler SideloadedHandler;
	Electra::FTimeValue LastPlaybackUpdateAbsPos;

	int32				Width = 0;
	int32				Height = 0;
	int32				TranslationX = 0;
	int32				TranslationY = 0;
	uint32				Timescale = 0;
};
