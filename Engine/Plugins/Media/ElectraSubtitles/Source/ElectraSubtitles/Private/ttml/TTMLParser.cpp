// Copyright Epic Games, Inc. All Rights Reserved.

#include "TTMLParser.h"
#include "TTMLXMLElements.h"
#include "TTMLSubtitleList.h"
#include "ElectraSubtitleUtils.h"
#include "expat.h"
#include "ttml/TTMLSubtitleHandler.h"
#include <expat_external.h>


#define XML_NAMESPACE_SEPARATOR_CHAR ':'

namespace ElectraTTMLParser
{

namespace ExpatMemHandler
{
	static void* MallocFn(size_t InSize)
	{
		return FMemory::Malloc(InSize);
	}
	static void* ReallocFn(void* InPtr, size_t InSize)
	{ 
		return FMemory::Realloc(InPtr, InSize);
	};
	static void FreeFn(void* InPtr)
	{ 
		FMemory::Free(InPtr);
	};

	static XML_Memory_Handling_Suite MemHooks =
	{
		ExpatMemHandler::MallocFn,
		ExpatMemHandler::ReallocFn,
		ExpatMemHandler::FreeFn
	};
}

struct ExpatUtils
{
	void Init()
	{
		CharSize = 0;
		LCharSize = 0;
		for(const XML_Feature* ExpatFeature = XML_GetFeatureList(); ExpatFeature; ++ExpatFeature)
		{
			if (ExpatFeature->feature == XML_FEATURE_END)
			{
				break;
			}
			else if (ExpatFeature->feature == XML_FEATURE_SIZEOF_XML_CHAR)
			{
				CharSize = (int32) ExpatFeature->value;
			}
			else if (ExpatFeature->feature == XML_FEATURE_SIZEOF_XML_LCHAR)
			{
				LCharSize = (int32) ExpatFeature->value;
			}
		}
		check(IsValid());
	}
	bool IsValid()
	{
		return CharSize == 1 && LCharSize == 1;
	}
	static ExpatUtils& Get()
	{
		static ExpatUtils Utils;
		Utils.Init();
		return Utils;
	}

	static FString ToFStringL(const XML_LChar* In)
	{
		FUTF8ToTCHAR Converted((const ANSICHAR*)In);
		return FString::ConstructFromPtrSize((TCHAR*)Converted.Get(), Converted.Length());
	}

	static FString ToFString(const XML_Char* In)
	{
		FUTF8ToTCHAR Converted((const ANSICHAR*)In);
		return FString::ConstructFromPtrSize((TCHAR*)Converted.Get(), Converted.Length());
	}

	static FString ToFString(const XML_Char* In, int32 Len)
	{
		FUTF8ToTCHAR Converted((const ANSICHAR*)In, Len);
		return FString::ConstructFromPtrSize((TCHAR*)Converted.Get(), Converted.Length());
	}

	static TArray<FXmlAttribute> ToAttributes(const XML_Char** InAtts)
	{
		TArray<FXmlAttribute> Attrs;
		while(InAtts && *InAtts)
		{
			FXmlAttribute& Attr = Attrs.AddDefaulted_GetRef();
			FString Name = ToFString(InAtts[0]);
			int32 NSPos = INDEX_NONE;
			if (Name.FindLastChar(TCHAR(XML_NAMESPACE_SEPARATOR_CHAR), NSPos))
			{
				Attr.Namespace = Name.Left(NSPos);
				Name.RightChopInline(NSPos + 1);
			}

			Attr.Name = Name;
			Attr.Value = ToFString(InAtts[1]);
			InAtts += 2;
		}
		return Attrs;
	}

	int32 CharSize = 0;
	int32 LCharSize = 0;
};

/*********************************************************************************************************************/

class FTTMLParser : public ITTMLParser
{
public:
	FTTMLParser();
	virtual ~FTTMLParser();

	virtual const FString& GetLastErrorMessage() const override
	{ return LastErrorMsg; }
	virtual bool ParseXMLDocument(const TArray<uint8>& InXMLDocumentData, const Electra::FParamDict& Options) override;
	virtual bool BuildSubtitleList(const Electra::FTimeValue& InDocumentStartTime, const Electra::FTimeValue& InDocumentDuration, const Electra::FParamDict& InOptions) override;
	virtual TSharedPtr<ITTMLSubtitleHandler, ESPMode::ThreadSafe> GetSubtitleHandler() override;

public:
	// Time related methods. To be called from the nodes during parsing only, not from the user.
	bool SetTimeBase(const FString& InTimeBase);
	void SetFrameRate(int32 InFrameRate);
	void SetSubFrameRate(int32 InSubFrameRate);
	void SetFrameRateMultiplier(int32 InNumerator, int32 InDenominator);
	void SetTickRate(uint32 InTickRate);
	void CalculateTickRate();
	bool GetTimeFromAttribute(Electra::FTimeValue& OutTime, ITTMLXMLElement* Element, const FXmlAttribute& InAttr);
	bool LexToUnsignedIntegerNumber(int64& Out, const FString& In, int32 NumDigits=0);
	bool LexToUnsignedFractional(Electra::FTimeFraction& OutTime, const FString& In);

private:
	struct FParseStackElement
	{
		FParseStackElement(const TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>& InElement)
			: Element(InElement)
		{ }
		TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> Element;
	};

	enum class ETimeBase
	{
		Media,
		Smpte,
		Clock,
		Invalid
	};

	friend class ITTMLXMLElement;
	void PushNewElement(const TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>& NewElement);
	void PopLastElement(const FString& ElementName);

	void ExpatStartElementHandler(const XML_Char* InName, const XML_Char** InAtts);
	void ExpatEndElementHandler(const XML_Char* InName);
	void ExpatCharacterDataHandler(const XML_Char* InS, int InLen);
	void ExpatStartNamespaceDeclHandler(const XML_Char* InPrefix, const XML_Char* InUri);
	void ExpatEndNamespaceDeclHandler(const XML_Char* InPrefix);

	static void _ExpatStartElementHandler(void* InUserData, const XML_Char* InName, const XML_Char** InAtts)
	{ reinterpret_cast<FTTMLParser*>(InUserData)->ExpatStartElementHandler(InName, InAtts); }
	static void _ExpatEndElementHandler(void* InUserData, const XML_Char* InName)
	{ reinterpret_cast<FTTMLParser*>(InUserData)->ExpatEndElementHandler(InName); }
	static void _ExpatCharacterDataHandler(void* InUserData, const XML_Char* InS, int InLen)
	{ reinterpret_cast<FTTMLParser*>(InUserData)->ExpatCharacterDataHandler(InS, InLen); }
	static void _ExpatStartNamespaceDeclHandler(void* InUserData, const XML_Char* InPrefix, const XML_Char* InUri)
	{ reinterpret_cast<FTTMLParser*>(InUserData)->ExpatStartNamespaceDeclHandler(InPrefix, InUri); }
	static void _ExpatEndNamespaceDeclHandler(void* InUserData, const XML_Char* InPrefix)
	{ reinterpret_cast<FTTMLParser*>(InUserData)->ExpatEndNamespaceDeclHandler(InPrefix); }


	TArray<FParseStackElement> ParseStack;
	XML_Parser Parser = nullptr;
	TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> RootElement;
	TSharedPtr<ITTMLSubtitleList, ESPMode::ThreadSafe> SubtitleList;
	FString LastErrorMsg;

	ETimeBase TimeBase = ETimeBase::Media;
	int32 TimeFrameRate = 30;
	int32 TimeSubFrameRate = 1;
	uint32 TimeTickRate = 0;
	Electra::FTimeFraction TimeFrameRateMultiplier {1,1};
};


/*********************************************************************************************************************/

TSharedPtr<ITTMLParser, ESPMode::ThreadSafe> ITTMLParser::Create()
{
	return MakeShared<FTTMLParser, ESPMode::ThreadSafe>();
}

FTTMLParser::FTTMLParser()
{
}

FTTMLParser::~FTTMLParser()
{
}

bool FTTMLParser::SetTimeBase(const FString& InTimeBase)
{
	if (ElectraSubtitleUtils::StringEquals(*InTimeBase, TEXT("media")))
	{
		TimeBase = ETimeBase::Media;
	}
	else if (ElectraSubtitleUtils::StringEquals(*InTimeBase, TEXT("smpte")))
	{
		TimeBase = ETimeBase::Smpte;
	}
	else if (ElectraSubtitleUtils::StringEquals(*InTimeBase, TEXT("clock")))
	{
		TimeBase = ETimeBase::Clock;
	}
	else
	{
		TimeBase = ETimeBase::Invalid;
		return false;
	}
	return true;
}

void FTTMLParser::SetFrameRate(int32 InFrameRate)
{
	check(InFrameRate);
	TimeFrameRate = InFrameRate;
}

void FTTMLParser::SetSubFrameRate(int32 InSubFrameRate)
{
	check(InSubFrameRate);
	TimeSubFrameRate = InSubFrameRate;
}

void FTTMLParser::SetTickRate(uint32 InTickRate)
{
	check(InTickRate);
	TimeTickRate = InTickRate;
}

void FTTMLParser::CalculateTickRate()
{
	if (TimeTickRate == 0)
	{
		TimeTickRate = TimeFrameRate * TimeSubFrameRate;
	}
}

void FTTMLParser::SetFrameRateMultiplier(int32 InNumerator, int32 InDenominator)
{
	check(InNumerator && InDenominator);
	TimeFrameRateMultiplier.SetFromND(InNumerator, (uint32)InDenominator);
}

bool FTTMLParser::GetTimeFromAttribute(Electra::FTimeValue& OutTime, ITTMLXMLElement* Element, const FXmlAttribute& Attr)
{
	// See: https://www.w3.org/TR/ttml2/#timing-time-value-expressions

	// Wallclock is not supported. Check for it first.
	if (ElectraSubtitleUtils::StringStartsWith(*Attr.GetValue(), TEXT("wallclock"), 9))
	{
		LastErrorMsg = FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d is not supported."), *Attr.GetValue(), *Attr.GetFullName(), *Element->GetName(), Element->GetLineNumber());
		return false;
	}
	// Only offset time ends with a metric
	enum class EMetric { h, m, s, ms, f, t, Invalid };
	auto GetOffsetMetric = [](FString& v) -> EMetric
	{
		check(v.Len() > 0);
		switch(GetData(v)[v.Len()-1])
		{
			case TCHAR('s'): 
				if (v.EndsWith(TEXT("ms"), ESearchCase::CaseSensitive))
				{
					v.LeftChopInline(2);
					return EMetric::ms;
				}
				v.LeftChopInline(1); 
				return EMetric::s;
			case TCHAR('m'): v.LeftChopInline(1); return EMetric::m;
			case TCHAR('h'): v.LeftChopInline(1); return EMetric::h;
			case TCHAR('f'): v.LeftChopInline(1); return EMetric::f;
			case TCHAR('t'): v.LeftChopInline(1); return EMetric::t;
			default: return EMetric::Invalid;
		}
	};

	FString Value(Attr.GetValue());
	EMetric OffsetMetric = GetOffsetMetric(Value);
	// Offset time?
	if (OffsetMetric != EMetric::Invalid)
	{
		Electra::FTimeFraction fr;
		if (LexToUnsignedFractional(fr, Value))
		{
			int64 n = fr.GetNumerator();
			uint32 d = fr.GetDenominator();
			// Scale by the metric.
			switch(OffsetMetric)
			{
				case EMetric::h:
					n *= 3600;
					break;
				case EMetric::m:
					n *= 60;
					break;
				case EMetric::s:
					break;
				case EMetric::ms:
					d *= 1000;
					break;
				case EMetric::f:
					OutTime.SetFromSeconds(fr.GetAsDouble() * TimeFrameRateMultiplier.GetDenominator() / ((double)TimeFrameRate * TimeFrameRateMultiplier.GetNumerator()));
					return true;
				case EMetric::t:
					OutTime.SetFromSeconds(fr.GetAsDouble() / TimeTickRate);
					return true;
			}
			OutTime.SetFromND(n, d);
			return true;
		}
	}
	// Clock time
	else
	{
		const TCHAR * const Delims[] = { TEXT(":") };
		TArray<FString> hmsf;
		int64 h=0, m=0;
		// Must at least have the three parts hours:minutes:seconds which must also have at least two digits each.
		Value.ParseIntoArray(hmsf, Delims, 1, true);
		if (hmsf.Num() == 3 && hmsf[0].Len() >= 2 && hmsf[1].Len() == 2 && hmsf[2].Len() >= 2)
		{
			Electra::FTimeFraction s;
			if (LexToUnsignedIntegerNumber(h, hmsf[0]) && LexToUnsignedIntegerNumber(m, hmsf[1]) && LexToUnsignedFractional(s, hmsf[2]))
			{
				OutTime.SetFromSeconds(h * 3600 + m * 60);
				OutTime += Electra::FTimeValue().SetFromTimeFraction(s);
				return true;
			}
		}
		else if (hmsf.Num() == 4 && hmsf[0].Len() >= 2 && hmsf[1].Len() == 2 && hmsf[2].Len() == 2)
		{
			int64 s=0;
			if (LexToUnsignedIntegerNumber(h, hmsf[0]) && LexToUnsignedIntegerNumber(m, hmsf[1]) && LexToUnsignedIntegerNumber(s, hmsf[2]))
			{
				// Check if the frames is followed by sub-frames.
				int32 SubFramePos = INDEX_NONE;
				int64 SubFrames=-1;
				if (hmsf[3].FindChar(TCHAR('.'), SubFramePos))
				{
					if (!LexToUnsignedIntegerNumber(SubFrames, hmsf[3].Mid(SubFramePos+1)))
					{
						LastErrorMsg = FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d could not be parsed."), *Attr.GetValue(), *Attr.GetFullName(), *Element->GetName(), Element->GetLineNumber());
						return false;
					}
					if (SubFrames >= TimeSubFrameRate)
					{
						LastErrorMsg = FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d has larger subframe number than specified subframe rate of %d allows for."), *Attr.GetValue(), *Attr.GetFullName(), *Element->GetName(), Element->GetLineNumber(), TimeSubFrameRate);
						return false;
					}
					hmsf[3].LeftInline(SubFramePos);
				}
				int64 frm=0;
				if (LexToUnsignedIntegerNumber(frm, hmsf[3]))
				{
					// Check that the frame number does not exceed the frame rate value.
					if (frm >= TimeFrameRate)
					{
						LastErrorMsg = FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d has larger frame number than specified frame rate of %d allows for."), *Attr.GetValue(), *Attr.GetFullName(), *Element->GetName(), Element->GetLineNumber(), TimeFrameRate);
						return false;
					}

					int64 TotalSeconds = h * 3600 + m * 60 + s;
					const double EffectiveFrameRate = TimeFrameRateMultiplier.GetAsDouble() * TimeFrameRate;
					double FrameSeconds = frm / EffectiveFrameRate;
					if (SubFrames > 0)
					{
						FrameSeconds += SubFrames / (EffectiveFrameRate * TimeSubFrameRate);
					}

					OutTime.SetFromSeconds(TotalSeconds + FrameSeconds);
					return true;
				}
			}
		}
	}
	LastErrorMsg = FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d could not be parsed."), *Attr.GetValue(), *Attr.GetFullName(), *Element->GetName(), Element->GetLineNumber());
	return false;
}

bool FTTMLParser::LexToUnsignedIntegerNumber(int64& Out, const FString& In, int32 NumDigits)
{
	bool bIsNegative = false;
	int64 v = 0;
	if (In.Len() > 0)
	{
		int32 Start = 0;
		/*
		if (In.Len() > 1)
		{
			if (In[Start] == TCHAR('-'))
			{
				bIsNegative = true;
				++Start;
			}
			else if (In[Start] == TCHAR('+'))
			{
				++Start;
			}
		}
		*/
		for(int32 i=Start, LastDigitPos = NumDigits ? (Start + NumDigits > In.Len() ? In.Len() : Start + NumDigits) : In.Len(); i<LastDigitPos; ++i)
		{
			v = v * 10;
			int32 d = (int32)In[i] - (int32)TCHAR('0');
			if (d < 0 || d > 9)
			{
				return false;
			}
			v += d;
		}
		Out = bIsNegative ? -v : v;
		return true;
	}
	return false;
}

bool FTTMLParser::LexToUnsignedFractional(Electra::FTimeFraction& OutTime, const FString& In)
{
	int64 n=0;
	uint32 d=1;

	// Find fractional dot, if any.
	int32 DotPos = INDEX_NONE;
	if (!In.FindChar(TCHAR('.'), DotPos))
	{
		if (LexToUnsignedIntegerNumber(n, In))
		{
			OutTime.SetFromND(n, d);
			return true;
		}
	}
	// Left and right side of the fractional dot must not be empty!
	else if (DotPos > 0 && DotPos < In.Len()-1)
	{
		const int64 kMaxValue = TNumericLimits<int64>::Max() / 10;
		if (LexToUnsignedIntegerNumber(n, In, DotPos))
		{
			// Locate the last digit that is not a zero (ignore trailing zeros)
			int32 LastDigitPos = In.Len() - 1;
			while(DotPos < LastDigitPos && In[LastDigitPos] == TCHAR('0'))
			{
				--LastDigitPos;
			}
			// Convert at most 7 fractional digits
			LastDigitPos = LastDigitPos > DotPos+7 ? DotPos+7 : LastDigitPos;
			for(int32 i=DotPos+1; i<=LastDigitPos; ++i)
			{
				// Check if the next multiplication would overflow our 64 bits.
				if (n > kMaxValue)
				{
					return false;
				}
				n = n * 10 + (In[i] - TCHAR('0'));
				d *= 10;
			}
			OutTime.SetFromND(n, n ? d : 1);
			return true;
		}
	}
	return false;
}

bool FTTMLParser::ParseXMLDocument(const TArray<uint8>& InXMLDocumentData, const Electra::FParamDict& Options)
{
	if (!ExpatUtils::Get().IsValid())
	{
		LastErrorMsg = TEXT("XML parser is not valid");
		return false;
	}

	// Create an intermediate 'root' element used during the parsing process.
	TSharedPtr<FTTMLXML_RootElement, ESPMode::ThreadSafe> Root = MakeShared<FTTMLXML_RootElement, ESPMode::ThreadSafe>(TEXT("<root>"));;
	RootElement = Root;

	Parser = XML_ParserCreate_MM(nullptr, &ExpatMemHandler::MemHooks, nullptr);
	XML_SetUserData(Parser, this);
	XML_SetElementHandler(Parser, _ExpatStartElementHandler, _ExpatEndElementHandler);
	XML_SetCharacterDataHandler(Parser, _ExpatCharacterDataHandler);
	//XML_SetNamespaceDeclHandler(Parser, _ExpatStartNamespaceDeclHandler, _ExpatEndNamespaceDeclHandler);
	XML_Status xs = XML_Parse(Parser, reinterpret_cast<const char*>(InXMLDocumentData.GetData()), InXMLDocumentData.Num(), 1);
	XML_Error xerr = XML_GetErrorCode(Parser);
	int32 ErrorLine = XML_GetCurrentLineNumber(Parser);
	XML_ParserFree(Parser);
	Parser = nullptr;

	if (xerr)
	{
		if (LastErrorMsg.IsEmpty())
		{
			FString ErrorMsg = ExpatUtils::ToFStringL(XML_ErrorString(xerr));
			LastErrorMsg = FString::Printf(TEXT("Parsing of TTML document failed with '%s' on line %d"), *ErrorMsg, ErrorLine);
		}
		RootElement.Reset();
		return false;
	}

	// Switch the root element over to the TT element and discard the intermediate node.
	RootElement = Root->GetTT();
	return true;
}

bool FTTMLParser::BuildSubtitleList(const Electra::FTimeValue& InDocumentStartTime, const Electra::FTimeValue& InDocumentDuration, const Electra::FParamDict& InOptions)
{
	LastErrorMsg.Empty();
	if (RootElement.IsValid())
	{
		TSharedPtr<ITTMLSubtitleList, ESPMode::ThreadSafe> NewSubtitleList = ITTMLSubtitleList::Create();
		if (NewSubtitleList->CreateFrom(MoveTemp(RootElement), InDocumentStartTime, InDocumentDuration, InOptions))
		{
			SubtitleList = MoveTemp(NewSubtitleList);
			return true;
		}
		else
		{
			LastErrorMsg = NewSubtitleList->GetLastErrorMessage();
		}
	}
	else
	{
		LastErrorMsg = TEXT("No valid parsed document");
	}
	return false;
}

TSharedPtr<ITTMLSubtitleHandler, ESPMode::ThreadSafe> FTTMLParser::GetSubtitleHandler()
{
	return StaticCastSharedPtr<ITTMLSubtitleHandler>(SubtitleList);
}

void FTTMLParser::PushNewElement(const TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>& NewElement)
{
	FParseStackElement e(NewElement);
	if (ParseStack.Num())
	{
		NewElement->SetParentElement(ParseStack.Last().Element);
	}
	ParseStack.Emplace(MoveTemp(e));
}

void FTTMLParser::PopLastElement(const FString& ElementName)
{
	check(ParseStack.Num());
	check(ElectraSubtitleUtils::StringEquals(*ElementName, *ParseStack.Last().Element->GetName()));
	ParseStack.Pop(EAllowShrinking::No);
}

void FTTMLParser::ExpatStartElementHandler(const XML_Char* InName, const XML_Char** InAtts)
{
	FString ElementName(ExpatUtils::ToFString(InName));

	TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> Element = ParseStack.Num() ? ParseStack.Last().Element : RootElement;
	if (!Element->ProcessElement(this, ElementName, ExpatUtils::ToAttributes(InAtts), (int32)XML_GetCurrentLineNumber(Parser)))
	{
		XML_StopParser(Parser, XML_FALSE);
	}
}

void FTTMLParser::ExpatEndElementHandler(const XML_Char* InName)
{
	FString ElementName(ExpatUtils::ToFString(InName));
	if (ParseStack.Num() == 0)
	{
		LastErrorMsg = FString::Printf(TEXT("Internal parser stack got mangled processing element <%s>"), *ElementName);
		XML_StopParser(Parser, XML_FALSE);
		return;
	}
	else
	{
		// Give the current element first crack at ending its processing.
		if (!ParseStack.Last().Element->ProcessClose(this, ElementName))
		{
			XML_StopParser(Parser, XML_FALSE);
		}
		PopLastElement(ElementName);
	}
}

void FTTMLParser::ExpatCharacterDataHandler(const XML_Char* InS, int InLen)
{
	if (ParseStack.Num())
	{
		FString Data(ExpatUtils::ToFString(InS, InLen));
		ParseStack.Last().Element->AppendData(MoveTemp(Data));
	}
}

void FTTMLParser::ExpatStartNamespaceDeclHandler(const XML_Char* InPrefix, const XML_Char* InUri)
{
	// Not currently used.
}

void FTTMLParser::ExpatEndNamespaceDeclHandler(const XML_Char* InPrefix)
{
	// Not currently used.
}

/*********************************************************************************************************************/

bool FTTMLTimeContainer::ProcessTimeAttributes(TArray<FXmlAttribute>& OutAttributes, ITTMLXMLElement* Element, FTTMLParser* Parser, const TArray<FXmlAttribute>& InAttributes)
{
	for(auto& Attr : InAttributes)
	{
		// See: https://www.w3.org/TR/ttml2/#timing-attribute-begin
		if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("begin")))
		{
			if (!Parser->GetTimeFromAttribute(Range.Start, Element, Attr))
			{
				return false;
			}
		}
		// See: https://www.w3.org/TR/ttml2/#timing-attribute-dur
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("dur")))
		{
			if (!Parser->GetTimeFromAttribute(Duration, Element, Attr))
			{
				return false;
			}
		}
		// See: https://www.w3.org/TR/ttml2/#timing-attribute-end
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("end")))
		{
			if (!Parser->GetTimeFromAttribute(Range.End, Element, Attr))
			{
				return false;
			}
		}
		// See: https://www.w3.org/TR/ttml2/#timing-attribute-timeContainer
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("timeContainer")))
		{
			if (!SetTimeContainerType(Parser, Attr.GetValue()))
			{
				return Element->SetError(Parser, FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d is bad."), *Attr.GetValue(), *Attr.GetFullName(), *Element->GetName(), Element->GetLineNumber()));
			}
		}
		else
		{
			OutAttributes.Emplace(Attr);
		}
	}
	return true;
}

bool FTTMLTimeContainer::SetTimeContainerType(FTTMLParser* Parser, const FString& InTimeContainerType)
{
	if (ElectraSubtitleUtils::StringEquals(*InTimeContainerType, TEXT("seq")))
	{
		TimeType = ETimeType::Seq;
		return true;
	}
	else if (ElectraSubtitleUtils::StringEquals(*InTimeContainerType, TEXT("par")))
	{
		TimeType = ETimeType::Par;
		return true;
	}
	return false;
}

void FTTMLTimeContainer::RecalculateTimesFromParent(FTTMLTimeContainer* InParent)
{
	// See: https://www.w3.org/TR/ttml2/#timing-attribute-vocabulary

	// If this is a 'Seq' container then we need to know about the sibling elements as they are then calculated
	// relative to their preceeding sibling with only the first calculated relative to the parent.
	check(TimeType == ETimeType::Par);
	if (InParent)
	{
		check(InParent->GetRange().IsValid());
		check(InParent->GetDuration().IsValid());

		// Apply start time of parent to this object.
		if (Range.Start.IsValid())
		{
			// Times are relative, so add them.
			Range.Start += InParent->GetBegin();
		}
		else
		{
			// When this time is not set it is implied to be zero for both 'par' and 'seq' containers
			// so we can just take the time from the parent.
			Range.Start = InParent->GetBegin();
		}

		// Likewise for the end.
		if (Range.End.IsValid())
		{
			// The end is a local time relative to the parent, so we have to add the START time of the parent!
			Range.End += InParent->GetBegin();
		}
		else if (Duration.IsValid())
		{
			Range.End = Range.Start + Duration;
		}
		else
		{
			Range.End = InParent->GetEnd();
		}

		// If a duration is specified and it is less than (Range.End-Range.Start) the end is calculated on the smaller duration.
		if (Duration.IsValid())
		{
			if (Duration < (Range.End - Range.Start))
			{
				Range.End = Range.Start + Duration;
			}
		}

		// Clamp end to the end time of the parent element.
		if (Range.End > InParent->GetEnd())
		{
			Range.End = InParent->GetEnd();
		}
		// The adjusted end could now be less than the start time. Clamp it if necessary.
		if (Range.End < Range.Start)
		{
			Range.End = Range.Start;
		}

		// Recalculate the duration.
		Duration = Range.End - Range.Start;
	}
}

/*********************************************************************************************************************/

ITTMLXMLElement::EType ITTMLXMLElement::GetType() const
{
	return Type;
}

void ITTMLXMLElement::SetParentElement(const TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>& InParentElement)
{
	ParentElement = InParentElement;
	// Inherit selected properties from the parent.
	if (InParentElement.IsValid())
	{
		SetPreserveSpace(InParentElement->GetPreserveSpace());
	}
}

bool ITTMLXMLElement::PushAndProcessNewElement(FTTMLParser* Parser, const TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>& Element, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber)
{
	Element->SetLineNumber(InLineNumber);
	Parser->PushNewElement(Element);
	return Element->ProcessAttributes(Parser, Attributes);
}

bool ITTMLXMLElement::SetError(FTTMLParser* Parser, FString ErrorMsg)
{
	Parser->LastErrorMsg = MoveTemp(ErrorMsg);
	return false;
}
	
const FString& ITTMLXMLElement::GetName() const
{
	return Name;
}

void ITTMLXMLElement::SetData(const FString& InData)
{
	Data = InData;
}

void ITTMLXMLElement::AppendData(const FString& InData)
{
	Data.Append(InData);
}

const FString& ITTMLXMLElement::GetData() const
{
	return Data;
}

void ITTMLXMLElement::SetLineNumber(int32 InLineNumber)
{
	LineNumber = InLineNumber;
}

int32 ITTMLXMLElement::GetLineNumber() const
{
	return LineNumber;
}

bool ITTMLXMLElement::ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber)
{
	OtherChildren.Emplace(MakeShared<FTTMLXML_OtherElement, ESPMode::ThreadSafe>(ElementName));
	return PushAndProcessNewElement(Parser, OtherChildren.Last(), Attributes, InLineNumber);
}

bool ITTMLXMLElement::ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& Attributes)
{
	for(auto& Attr : Attributes)
	{
		if (ElectraSubtitleUtils::StringEquals(*Attr.GetNamespace(), TEXT("xml")) && ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("space")))
		{
			bPreserveSpace = ElectraSubtitleUtils::StringEquals(*Attr.GetValue(), TEXT("preserve"));
		}
		else
		{
			OtherAttributes.Emplace(Attr);
		}
	}
	return true;
}

bool ITTMLXMLElement::ProcessClose(FTTMLParser* Parser, const FString& ElementName)
{
	// Any element data that an element did not process we discard.
	// This may not be universally useful, but we are processing text here and elements that did not care
	// about the text do not have to store all the document whitespaces.
	Data.Empty();
	return true;
}

void ITTMLXMLElement::RecalculateTimes(bool bRecursively)
{
	if (GetTimeContainer())
	{
		TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> Parent = GetParentElement();
		if (Parent.IsValid() && Parent->GetTimeContainer())
		{
			GetTimeContainer()->RecalculateTimesFromParent(Parent->GetTimeContainer());
		}
	}
}

/*********************************************************************************************************************/

bool FTTMLXML_RootElement::ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber)
{
	if (!ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("tt")))
	{
		return SetError(Parser, FString::Printf(TEXT("TTML document's root element is not <tt> but <%s> in line %d"), *ElementName, InLineNumber));
	}
	TT = MakeShared<FTTMLXML_TTElement, ESPMode::ThreadSafe>(ElementName);
	return PushAndProcessNewElement(Parser, TT, Attributes, InLineNumber);
}

/*********************************************************************************************************************/

bool FTTMLXML_TTElement::ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber)
{
	// If there is no @tickRate specified we need to calculate it based on @frameRate and @subFrameRate
	Parser->CalculateTickRate();

	if (ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("head")))
	{
		Head = MakeShared<FTTMLXML_HeadElement, ESPMode::ThreadSafe>(ElementName);
		return PushAndProcessNewElement(Parser, Head, Attributes, InLineNumber);
	}
	else if (ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("body")))
	{
		Body = MakeShared<FTTMLXML_BodyElement, ESPMode::ThreadSafe>(ElementName);
		return PushAndProcessNewElement(Parser, Body, Attributes, InLineNumber);
	}
	else
	{
		return Super::ProcessElement(Parser, ElementName, Attributes, InLineNumber);
	}
}

bool FTTMLXML_TTElement::ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& AllAttributes)
{
	TArray<FXmlAttribute> Attributes;
	for(int32 i=0; i<AllAttributes.Num(); ++i)
	{
		const FXmlAttribute& Attr = AllAttributes[i];

		// See: https://www.w3.org/TR/ttml2/#parameter-attribute-timeBase
		if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("timeBase")))
		{
			if (!Parser->SetTimeBase(Attr.GetValue()))
			{
				return SetError(Parser, FString::Printf(TEXT("TTML attribute %s with value %s on element %s in line %d is not supported"), *Attr.GetFullName(), *Attr.GetValue(), *GetName(), GetLineNumber()));
			}
		}
		// See: https://www.w3.org/TR/ttml2/#parameter-attribute-frameRate
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("frameRate")))
		{
			int64 v=0;
			if (Parser->LexToUnsignedIntegerNumber(v, Attr.GetValueNoWS()))
			{
				// Must be greater than zero.
				if (v > 0)
				{
					Parser->SetFrameRate((int32) v);
					continue;
				}
			}
			return SetError(Parser, FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d is bad."), *Attr.GetValue(), *Attr.GetFullName(), *GetName(), GetLineNumber()));
		}
		// See: https://www.w3.org/TR/ttml2/#parameter-attribute-frameRateMultiplier
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("frameRateMultiplier")))
		{
			// This is a whitespace separated pair of integers.
			TArray<FString> nd;
			Attr.GetValue().ParseIntoArrayWS(nd, nullptr, true);
			if (nd.Num() == 2)
			{
				int64 n=0, d=0;
				if (Parser->LexToUnsignedIntegerNumber(n, nd[0]) && Parser->LexToUnsignedIntegerNumber(d, nd[1]))
				{
					if (n > 0 && d > 0)
					{
						Parser->SetFrameRateMultiplier((int32)n, (int32)d);
						continue;
					}
				}
			}
			return SetError(Parser, FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d is bad."), *Attr.GetValue(), *Attr.GetFullName(), *GetName(), GetLineNumber()));
		}
		// See: https://www.w3.org/TR/ttml2/#parameter-attribute-clockMode
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("clockMode")))
		{
			return SetError(Parser, FString::Printf(TEXT("TTML attribute %s on element %s in line %d is not supported"), *Attr.GetFullName(), *GetName(), GetLineNumber()));
		}
		// See: https://www.w3.org/TR/ttml2/#parameter-attribute-subFrameRate
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("subFrameRate")))
		{
			int64 v=0;
			if (Parser->LexToUnsignedIntegerNumber(v, Attr.GetValueNoWS()))
			{
				// Must be greater than zero.
				if (v > 0)
				{
					Parser->SetSubFrameRate((int32) v);
					continue;
				}
			}
			return SetError(Parser, FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d is bad."), *Attr.GetValue(), *Attr.GetFullName(), *GetName(), GetLineNumber()));
		}
		// See: https://www.w3.org/TR/ttml2/#parameter-attribute-dropMode
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("dropMode")))
		{
			return SetError(Parser, FString::Printf(TEXT("TTML attribute %s on element %s in line %d is not supported"), *Attr.GetFullName(), *GetName(), GetLineNumber()));
		}
		// See: https://www.w3.org/TR/ttml2/#parameter-attribute-tickRate
		else if (ElectraSubtitleUtils::StringEquals(*Attr.GetName(), TEXT("tickRate")))
		{
			int64 v=0;
			if (Parser->LexToUnsignedIntegerNumber(v, Attr.GetValueNoWS()))
			{
				// Must be greater than zero.
				if (v > 0)
				{
					Parser->SetTickRate((uint32) v);
					continue;
				}
			}
			return SetError(Parser, FString::Printf(TEXT("TTML attribute value %s of attribute %s on element %s in line %d is bad."), *Attr.GetValue(), *Attr.GetFullName(), *GetName(), GetLineNumber()));
		}
		else
		{
			Attributes.Add(Attr);
		}
	}
	return Super::ProcessAttributes(Parser, Attributes);
}

/*********************************************************************************************************************/

bool FTTMLXML_BodyElement::ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber)
{
	if (ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("div")))
	{
		Divs.Emplace(MakeShared<FTTMLXML_DivElement, ESPMode::ThreadSafe>(ElementName));
		return PushAndProcessNewElement(Parser, Divs.Last(), Attributes, InLineNumber);
	}
	else
	{
		return Super::ProcessElement(Parser, ElementName, Attributes, InLineNumber);
	}
}

bool FTTMLXML_BodyElement::ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& AllAttributes)
{
	TArray<FXmlAttribute> Attributes;
	if (!ProcessTimeAttributes(Attributes, this, Parser, AllAttributes))
	{
		return false;
	}
	return Super::ProcessAttributes(Parser, Attributes);
}

void FTTMLXML_BodyElement::RecalculateTimes(bool bRecursively)
{
	// First recalculate our time via the parent element before processing our child elements.
	Super::RecalculateTimes(false);
	if (bRecursively)
	{
		for(auto &Div : Divs)
		{
			Div->RecalculateTimes(bRecursively);
		}
	}
}

void FTTMLXML_BodyElement::CollectContentElementRoots(TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> &OutContentElements, bool bWithPositiveDurationOnly)
{
	for(auto &Div : Divs)
	{
		if (bWithPositiveDurationOnly && Div->GetDuration() <= Electra::FTimeValue::GetZero())
		{
			continue;
		}
		Div->CollectContentElementRoots(OutContentElements, bWithPositiveDurationOnly);
	}
}

/*********************************************************************************************************************/

bool FTTMLXML_DivElement::ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber)
{
	if (ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("p")))
	{
		ContentElements.Emplace(MakeShared<FTTMLXML_P_Element, ESPMode::ThreadSafe>(ElementName));
		return PushAndProcessNewElement(Parser, ContentElements.Last(), Attributes, InLineNumber);
	}
	else if (ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("span")))
	{
		ContentElements.Emplace(MakeShared<FTTMLXML_Span_Element, ESPMode::ThreadSafe>(ElementName));
		return PushAndProcessNewElement(Parser, ContentElements.Last(), Attributes, InLineNumber);
	}
	else
	{
		return Super::ProcessElement(Parser, ElementName, Attributes, InLineNumber);
	}
}

bool FTTMLXML_DivElement::ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& AllAttributes)
{
	// We only process the timing related attributes on the <div> element.
	TArray<FXmlAttribute> Attributes;
	if (!ProcessTimeAttributes(Attributes, this, Parser, AllAttributes))
	{
		return false;
	}
	return Super::ProcessAttributes(Parser, Attributes);
}

void FTTMLXML_DivElement::RecalculateTimes(bool bRecursively)
{
	// First recalculate our time via the parent element before processing our child elements.
	Super::RecalculateTimes(false);
	if (bRecursively)
	{
		for(auto &Content : ContentElements)
		{
			Content->RecalculateTimes(bRecursively);
		}
	}
}

void FTTMLXML_DivElement::CollectContentElementRoots(TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> &OutContentElements, bool bWithPositiveDurationOnly)
{
	if (bWithPositiveDurationOnly)
	{
		for(auto& Content : ContentElements)
		{
			if (Content->GetDuration() > Electra::FTimeValue::GetZero())
			{
				OutContentElements.Emplace(Content);
			}
		}
	}
	else
	{
		OutContentElements.Append(ContentElements);
	}
}

/*********************************************************************************************************************/

bool FTTMLXML_ContentElement::ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber)
{
	// If we have already collected text we need to create an anonymous <span> for it.
	// See end of: https://www.w3.org/TR/ttml2/#content-vocabulary-span
	//        and: https://www.w3.org/TR/ttml2/#procedure-construct-anonymous-spans
	if (Data.Len())
	{
		TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe> AnonymousSpan = MakeShared<FTTMLXML_ContentElement, ESPMode::ThreadSafe>(TEXT("anonymous"), EType::AnonymousSpan);
		AnonymousSpan->SetParentElement(AsShared());
		AnonymousSpan->SetText(MoveTemp(Data));
		Children.Emplace(MoveTemp(AnonymousSpan));
	}
	if (ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("br")))
	{
		Children.Emplace(MakeShared<FTTMLXML_BR_Element, ESPMode::ThreadSafe>(ElementName));
		return PushAndProcessNewElement(Parser, Children.Last(), Attributes, InLineNumber);
	}
	else if (ElectraSubtitleUtils::StringEquals(*ElementName, TEXT("span")))
	{
		Children.Emplace(MakeShared<FTTMLXML_Span_Element, ESPMode::ThreadSafe>(ElementName));
		return PushAndProcessNewElement(Parser, Children.Last(), Attributes, InLineNumber);
	}
	else
	{
		return Super::ProcessElement(Parser, ElementName, Attributes, InLineNumber);
	}
}

bool FTTMLXML_ContentElement::ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& AllAttributes)
{
	// We only process the timing related attributes on the <div> element.
	TArray<FXmlAttribute> Attributes;
	if (!ProcessTimeAttributes(Attributes, this, Parser, AllAttributes))
	{
		return false;
	}
	return Super::ProcessAttributes(Parser, Attributes);
}

bool FTTMLXML_ContentElement::ProcessClose(FTTMLParser* Parser, const FString& ElementName)
{
	if (Data.Len())
	{
		if (Children.Num())
		{
			TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe> AnonymousSpan = MakeShared<FTTMLXML_ContentElement, ESPMode::ThreadSafe>(TEXT("anonymous"), EType::AnonymousSpan);
			AnonymousSpan->SetParentElement(AsShared());
			AnonymousSpan->SetText(MoveTemp(Data));
			Children.Emplace(MoveTemp(AnonymousSpan));
			MergeChildren();
		}
		else
		{
			SetText(MoveTemp(Data));
		}
	}
	return Super::ProcessClose(Parser, ElementName);
}

void FTTMLXML_ContentElement::SetText(FString InText)
{
	if (GetPreserveSpace())
	{
		Text = MoveTemp(InText);
	}
	else
	{
		// See: https://developer.mozilla.org/en-US/docs/Web/API/Document_Object_Model/Whitespace
		// Ignore spaces before newlines.
		while(InText.ReplaceInline(TEXT(" \n"), TEXT("\n"), ESearchCase::CaseSensitive))
		{
		}
		// Ignore spaces after newlines.
		while(InText.ReplaceInline(TEXT("\n "), TEXT("\n"), ESearchCase::CaseSensitive))
		{
		}
		// Replace tab with space.
		InText.ReplaceCharInline(TCHAR('\t'), TCHAR(' '), ESearchCase::CaseSensitive);
		// Replace linefeed with space.
		InText.ReplaceCharInline(TCHAR('\n'), TCHAR(' '), ESearchCase::CaseSensitive);
		// Replace consecutive spaces with a single space.
		while(InText.ReplaceInline(TEXT("  "), TEXT(" "), ESearchCase::CaseSensitive))
		{
		}
		// We do NOT remove leading and trailing space here because this element may
		// have siblings and removal applies only to the total combined string from all siblings.
		Text = MoveTemp(InText);
	}
}


void FTTMLXML_ContentElement::RecalculateTimes(bool bRecursively)
{
	// First recalculate our time via the parent element before processing our child elements.
	Super::RecalculateTimes(false);
	if (bRecursively)
	{
		for(auto &Child : Children)
		{
			Child->RecalculateTimes(bRecursively);
		}
	}
}

void FTTMLXML_ContentElement::MergeChildren()
{
	// See: https://www.w3.org/TR/ttml2/#procedure-construct-anonymous-spans
	// Merge anonymous spans
	for(int32 i=1; i<Children.Num(); ++i)
	{
		if (Children[i]->GetType() == EType::AnonymousSpan && Children[i-1]->GetType() == EType::AnonymousSpan)
		{
			Children[i-1]->Text.Append(Children[i]->Text);
			Children.RemoveAt(i);
			--i;
		}
	}
	// Only one anonymous span?
	if (Children.Num() == 1 && Children[0]->GetType() == EType::AnonymousSpan)
	{
		Text.Append(Children[0]->Text);
		Children.Empty();
	}
}

/*********************************************************************************************************************/

}
