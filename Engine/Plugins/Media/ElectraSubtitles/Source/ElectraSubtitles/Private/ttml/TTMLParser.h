// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace Electra { class FParamDict; }
namespace Electra { class FTimeValue; }

namespace ElectraTTMLParser
{
class ITTMLSubtitleHandler;

class ITTMLParser
{
public:
	static TSharedPtr<ITTMLParser, ESPMode::ThreadSafe> Create();
	
	virtual ~ITTMLParser() = default;

	/**
	 * Returns the most recent error message.
	 */
	virtual const FString& GetLastErrorMessage() const = 0;

	/**
	 * Parses the provided XML as a TTML document.
	 * 
	 * @return true if successful, false if not. Get error message from GetLastErrorMessage().
	 */ 
	virtual bool ParseXMLDocument(const TArray<uint8>& InXMLDocumentData, const Electra::FParamDict& InOptions) = 0;

	/**
	 * Creates an internal list of subtitles sorted by time.
	 * Call this after a successful parse of the document.
	 * 
	 * NOTE: 
	 *    This call will modify the parsed document in-place. This can be called only once.
	 * 
	 * @param InDocumentStartTime The base start time to offset internal timestamps with
	 * @param InDocumentDuration The duration this document is valid for. Subtitles falling outside the document range will not be used.
	 * @param InOptions Additional options:
	 *                  "sendEmptySubtitleDuringGaps"(bool) : Set to true to have an empty subtitle being sent in between actual subtitles
	 *                                                        Useful to remove subtitles from the screen without considering start times and display durations.
	 * 
	 * @return true if successful, false otherwise.
	 */
	virtual bool BuildSubtitleList(const Electra::FTimeValue& InDocumentStartTime, const Electra::FTimeValue& InDocumentDuration, const Electra::FParamDict& InOptions) = 0;

	/**
	 * Get the handler for the list of subtitles after a successful call to BuildSubtitleList().
	 * With this handler retrieved and kept it is ok to destroy this parser.
	 * 
	 * @return Handler to the list of parsed subtitles within the active document time range.
	 */
	virtual TSharedPtr<ITTMLSubtitleHandler, ESPMode::ThreadSafe> GetSubtitleHandler() = 0;
};

}
