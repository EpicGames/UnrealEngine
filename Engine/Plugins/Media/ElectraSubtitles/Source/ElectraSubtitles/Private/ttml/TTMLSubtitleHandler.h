// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PlayerTime.h"

struct FScriptContainerElement;

namespace ElectraTTMLParser
{

class ITTMLSubtitleHandler
{
public:
	virtual ~ITTMLSubtitleHandler() = default;

	struct FActiveSubtitle
	{
		FString Text;
		Electra::FTimeRange TimeRange;
		bool bIsEmptyGap = false;
	};


	// Resets the current active range on a time discontinuity (eg. seeking).
	virtual void ClearActiveRange() = 0;

	// Updates the active subtitle range. Returns true if changed, false if not changed.
	virtual bool UpdateActiveRange(const Electra::FTimeRange& InRange) = 0;

	// Returns the active subtitle.
	virtual void GetActiveSubtitles(TArray<FActiveSubtitle>& OutSubtitles) const = 0;
};


}
