// Copyright Epic Games, Inc. All Rights Reserved.

#include "TTMLSubtitleList.h"

#include "ParameterDictionary.h"
#include "TTMLXMLElements.h"
#include "ttml/TTMLSubtitleHandler.h"

namespace ElectraTTMLParser
{

namespace OptionKeys
{
	static const FName SendEmptySubtitleDuringGaps(TEXT("sendEmptySubtitleDuringGaps"));
}


class FTTMLSubtitleList : public TSharedFromThis<FTTMLSubtitleList, ESPMode::ThreadSafe>, public ITTMLSubtitleList
{
public:
	virtual ~FTTMLSubtitleList() = default;

	//--------------------------------------
	// Methods from ITTMLSubtitleList
	//
	virtual const FString& GetLastErrorMessage() const override
	{ return LastErrorMessage; }
	virtual bool CreateFrom(TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> InTTMLTT, const Electra::FTimeValue& InDocumentStartTime, const Electra::FTimeValue& InDocumentDuration, const Electra::FParamDict& InOptions) override;


	//--------------------------------------
	// Methods from ITTMLSubtitleHandler
	//
	virtual void ClearActiveRange() override;
	virtual bool UpdateActiveRange(const Electra::FTimeRange& InRange) override;
	virtual void GetActiveSubtitles(TArray<FActiveSubtitle>& OutSubtitles) const override;

private:
	struct FPartialSubtitle
	{
		FString Text;
		Electra::FTimeRange TimeRange;
		bool bPreserveSpace = false;
	};

	void BuildActiveSubtitleFrom(TArray<FPartialSubtitle>& OutPartial, Electra::FTimeValue& OutValidUntil, const TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>& Element, const Electra::FTimeValue& AtTime);

	bool bSendEmptySubtitleDuringGaps = false;

	FString LastErrorMessage;
	TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> Elements;

	TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> ActiveElements;
	Electra::FTimeValue MostRecentActivatedStartTime;
	Electra::FTimeValue NextElementStartTime;

	TArray<FActiveSubtitle> ActiveSubtitles;
	Electra::FTimeValue ActiveSubtitlesValidUntil;
	int32 ActiveSubtitleFirstElementIndex = 0;
	bool bSentActiveSubtitle = false;

};

/*********************************************************************************************************************/

TSharedPtr<ITTMLSubtitleList, ESPMode::ThreadSafe> ITTMLSubtitleList::Create()
{
	return MakeShared<FTTMLSubtitleList, ESPMode::ThreadSafe>();
}

/*********************************************************************************************************************/

bool FTTMLSubtitleList::CreateFrom(TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> InTTMLTT, const Electra::FTimeValue& InDocumentStartTime, const Electra::FTimeValue& InDocumentDuration, const Electra::FParamDict& InOptions)
{
	ClearActiveRange();

	if (!InTTMLTT.IsValid() || InTTMLTT->GetType() != ITTMLXMLElement::EType::TT)
	{
		LastErrorMessage = TEXT("No valid document or not a TT element");
		return false;
	}

	FTTMLXML_TTElement* TTMLTT = static_cast<FTTMLXML_TTElement*>(InTTMLTT.Get());
	if (!TTMLTT->GetBody().IsValid())
	{
		LastErrorMessage = TEXT("Document has no <body> element");
		return false;
	}

	bSendEmptySubtitleDuringGaps = InOptions.GetValue(OptionKeys::SendEmptySubtitleDuringGaps).SafeGetBool();

	// Set the root temporal extent of the document.
	TTMLTT->SetBegin(InDocumentStartTime);
	TTMLTT->SetDuration(InDocumentDuration);
	TTMLTT->SetEnd(InDocumentStartTime + InDocumentDuration);
	// Calculate the start & end times of all elements recursively.
	TTMLTT->GetBody()->RecalculateTimes(true);

	// Collect the root content elements from all <div>s.
	// Only the <p> or <span> element is collected, NOT their children, if any.
	// Also, only those that have a positive duration are collected.
	TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> ContentElements;
	TTMLTT->GetBody()->CollectContentElementRoots(ContentElements, true);

	// Sort them by ascending start time.
	ContentElements.StableSort([](const TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>& e1, const TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>& e2)
	{ 
		return e1->GetBegin() < e2->GetBegin();		
	});

	// All well, take on the list.
	Elements = MoveTemp(ContentElements);
	return true;
}


/*********************************************************************************************************************/

void FTTMLSubtitleList::BuildActiveSubtitleFrom(TArray<FPartialSubtitle>& OutPartial, Electra::FTimeValue& OutValidUntil, const TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>& Element, const Electra::FTimeValue& AtTime)
{
	// Add the element text and time range to the partial subtitle output list, if the text is not empty.
	if (Element->GetText().Len())
	{
		FPartialSubtitle Subtitle;
		Subtitle.Text = Element->GetText();
		Subtitle.bPreserveSpace = Element->GetPreserveSpace();
		Subtitle.TimeRange = Element->GetRange();
		OutPartial.Emplace(MoveTemp(Subtitle));
	}
	// Update the valid-until time with the smaller (or the only) end time.
	if (!OutValidUntil.IsValid() || OutValidUntil > Element->GetRange().End)
	{
		OutValidUntil = Element->GetRange().End;
	}
	// Collect the children recursively.
	const TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>>& Children = Element->GetChildren();
	for(auto& Child : Children)
	{
		if (Child->GetRange().Contains(AtTime))
		{
			BuildActiveSubtitleFrom(OutPartial, OutValidUntil, Child, AtTime);
		}
		// If there is a child element in the future then the current subtitle is only good until then.
		else if (Child->GetRange().Start > AtTime & Child->GetRange().Start < OutValidUntil)
		{
			OutValidUntil = Child->GetRange().Start;
		}
	}
}


void FTTMLSubtitleList::ClearActiveRange()
{
	ActiveElements.Empty();
	MostRecentActivatedStartTime.SetToInvalid();
	NextElementStartTime.SetToInvalid();

	ActiveSubtitles.Empty();
	ActiveSubtitlesValidUntil.SetToInvalid();
	ActiveSubtitleFirstElementIndex = 0;
	bSentActiveSubtitle = false;
}

bool FTTMLSubtitleList::UpdateActiveRange(const Electra::FTimeRange& InRange)
{
	// Short circuit processing if there are no subtitles.
	if (Elements.Num() == 0)
	{
		return false;
	}

	bool bIsDirty = false;
	
	// Check if any of the active elements are now no longer active and remove them.
	for(int32 i=0; i<ActiveElements.Num(); ++i)
	{
		if (!ActiveElements[i]->GetRange().Overlaps(InRange))
		{
			bIsDirty = true;
			ActiveElements.RemoveAt(i);
			if (--i>=0 && ActiveElements[i]->GetBegin() > MostRecentActivatedStartTime)
			{
				MostRecentActivatedStartTime = ActiveElements[i]->GetBegin();
			}
		}
	}
	// Find the next elements to activate.
	int32 NextActiveIndex;
	bool bIsFirstNew = true;
	for(NextActiveIndex=ActiveSubtitleFirstElementIndex; NextActiveIndex<Elements.Num(); ++NextActiveIndex)
	{
		if (Elements[NextActiveIndex]->GetRange().Overlaps(InRange))
		{
			if (bIsFirstNew)
			{
				bIsFirstNew = false;
				MostRecentActivatedStartTime = Elements[NextActiveIndex]->GetBegin();
			}
			if (!ActiveElements.Contains(Elements[NextActiveIndex]))
			{
				ActiveElements.Emplace(Elements[NextActiveIndex]);
				bIsDirty = true;
			}
		}
		else if (Elements[NextActiveIndex]->GetBegin() > InRange.End)
		{
			NextElementStartTime = Elements[NextActiveIndex]->GetBegin();
			break;
		}
	}
	ActiveSubtitleFirstElementIndex = NextActiveIndex;
	if (NextActiveIndex == Elements.Num())
	{
		NextElementStartTime.SetToInvalid();
	}


	// If the list of active elements did not change and we have a time until which the current set of subtitles is valid
	// we don't have to update anything until that time.
	if (!bIsDirty && ActiveSubtitlesValidUntil.IsValid() && InRange.End <= ActiveSubtitlesValidUntil)
	{
		return false;
	}


	if (ActiveElements.Num())
	{
		TArray<FActiveSubtitle> NewActiveSubtitles;
		Electra::FTimeValue SmallestValidUntil;
		for(auto& Active : ActiveElements)
		{
			Electra::FTimeValue ValidUntil;
			TArray<FPartialSubtitle> Partials;
			BuildActiveSubtitleFrom(Partials, ValidUntil, Active, InRange.End);
			if (Partials.Num())
			{
				FActiveSubtitle Subtitle;
				Subtitle.TimeRange.Start = MostRecentActivatedStartTime;
				Subtitle.TimeRange.End = ValidUntil;
				if (NextElementStartTime.IsValid() && NextElementStartTime < ValidUntil)
				{
					Subtitle.TimeRange.End = NextElementStartTime;
				}

				// See: https://developer.mozilla.org/en-US/docs/Web/API/Document_Object_Model/Whitespace
				for(int32 i=0; i<Partials.Num(); ++i)
				{
					if (i == 0)
					{
						Subtitle.Text.Append(Partials[i].Text);
					}
					else
					{
						// If this part preserves space and the former did too we concatenate.
						if (Partials[i].bPreserveSpace && Partials[i-1].bPreserveSpace)
						{
							Subtitle.Text.Append(Partials[i].Text);
						}
						// Does this part preserve spaces and the former does not?
						else if (Partials[i].bPreserveSpace && !Partials[i-1].bPreserveSpace)
						{
							// Check if this begins with a space and the combined so far ends with one.
							if (Partials[i].Text.Len() && Partials[i].Text[0] == TCHAR(' ') &&
								Subtitle.Text.Len() && Subtitle.Text[Subtitle.Text.Len() - 1] == TCHAR(' '))
							{
								Subtitle.Text.TrimEndInline();
							}
							Subtitle.Text.Append(Partials[i].Text);
						}
						// Does this part not preserve spaces and the former does?
						else if (!Partials[i].bPreserveSpace && Partials[i-1].bPreserveSpace)
						{
							Subtitle.Text.Append(Partials[i].Text.TrimStart());
						}
						// This part and the former do not preserve spaces.
						else
						{
							bool bHasSpace = (Partials[i].Text.Len() && Partials[i].Text[0] == TCHAR(' ')) || (Subtitle.Text.Len() && Subtitle.Text[Subtitle.Text.Len() - 1] == TCHAR(' '));
							Subtitle.Text.TrimEndInline();
							if (bHasSpace)
							{
								Subtitle.Text.AppendChar(TCHAR(' '));
							}
							Subtitle.Text.Append(Partials[i].Text.TrimStart());
						}
					}
				}
				if (!Partials[0].bPreserveSpace)
				{
					Subtitle.Text.TrimStartInline();
				}
				if (!Partials.Last().bPreserveSpace)
				{
					Subtitle.Text.TrimEndInline();
				}

				NewActiveSubtitles.Emplace(MoveTemp(Subtitle));
			}

			if (!SmallestValidUntil.IsValid() || ValidUntil < SmallestValidUntil)
			{
				SmallestValidUntil = ValidUntil;
			}
		}
		ActiveSubtitlesValidUntil = SmallestValidUntil;
		ActiveSubtitles = MoveTemp(NewActiveSubtitles);
		bSentActiveSubtitle = true;
		return true;
	}
	else
	{
		if (bSendEmptySubtitleDuringGaps && bSentActiveSubtitle)
		{
			bSentActiveSubtitle = false;

			FActiveSubtitle EmptySubtitle;
			EmptySubtitle.bIsEmptyGap = true;
			EmptySubtitle.TimeRange.Start = ActiveSubtitlesValidUntil.IsValid() ? ActiveSubtitlesValidUntil : InRange.Start;
			EmptySubtitle.TimeRange.End = NextElementStartTime.IsValid() ? NextElementStartTime : EmptySubtitle.TimeRange.Start + Electra::FTimeValue(0.1);
			// Make the empty subtitle last for no more than 1 second.
			Electra::FTimeValue Dur = EmptySubtitle.TimeRange.End - EmptySubtitle.TimeRange.Start;
			if (Dur.GetAsSeconds() > 1.0 || Dur < Electra::FTimeValue::GetZero())
			{
				EmptySubtitle.TimeRange.End = EmptySubtitle.TimeRange.Start + Electra::FTimeValue(1.0);
			}
			ActiveSubtitles.Empty();
			ActiveSubtitles.Emplace(MoveTemp(EmptySubtitle));
			ActiveSubtitlesValidUntil.SetToInvalid();
			return true;
		}
		else
		{
			ActiveSubtitles.Empty();
			ActiveSubtitlesValidUntil.SetToInvalid();
			return false;
		}
	}
}

void FTTMLSubtitleList::GetActiveSubtitles(TArray<FActiveSubtitle>& OutSubtitles) const
{
	OutSubtitles = ActiveSubtitles;
}

}

