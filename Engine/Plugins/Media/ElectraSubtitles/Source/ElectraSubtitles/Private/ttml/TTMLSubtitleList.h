// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TTMLSubtitleHandler.h"
#include "Templates/SharedPointer.h"

namespace Electra { class FParamDict; }
namespace Electra { class FTimeValue; }

namespace ElectraTTMLParser
{
class ITTMLXMLElement;

class ITTMLSubtitleList : public ITTMLSubtitleHandler
{
public:
	static TSharedPtr<ITTMLSubtitleList, ESPMode::ThreadSafe> Create();
	
	virtual ~ITTMLSubtitleList() = default;
	virtual const FString& GetLastErrorMessage() const = 0;

	virtual bool CreateFrom(TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> InTTMLTT, const Electra::FTimeValue& InDocumentStartTime, const Electra::FTimeValue& InDocumentDuration, const Electra::FParamDict& InOptions) = 0;
};

}
