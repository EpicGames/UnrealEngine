// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlayerTime.h"
#include "Templates/SharedPointer.h"

namespace ElectraTTMLParser
{
class ITTMLXMLElement;
class FTTMLParser;


struct FXmlAttribute
{
	const FString& GetName() const 	{ return Name; }
	const FString& GetValue() const	{ return Value; }
	const FString& GetNamespace() const	{ return Namespace; }

	FString GetFullName() const
	{
		if (Namespace.Len())
		{
			return FString::Printf(TEXT("%s:%s"), *Namespace, *Name);
		}
		return Name;
	}

	FString GetValueNoWS() const
	{ 
		return Value.TrimStartAndEnd();
	}

	FString	Name;
	FString Value;
	FString Namespace;
};


class FTTMLTimeContainer
{
public:
	enum class ETimeType
	{
		Par,
		Seq
	};


	ETimeType GetTimeType() const
	{ return TimeType; }

	Electra::FTimeRange GetRange() const
	{ return Range; }

	Electra::FTimeValue GetBegin() const
	{ return Range.Start; }
	void SetBegin(const Electra::FTimeValue& InBegin)
	{ Range.Start = InBegin; }

	Electra::FTimeValue GetEnd() const
	{ return Range.End; }
	void SetEnd(const Electra::FTimeValue& InEnd)
	{ Range.End = InEnd; }

	Electra::FTimeValue GetDuration() const
	{ return Duration; }
	void SetDuration(const Electra::FTimeValue& InDuration)
	{ Duration = InDuration; }

	bool ProcessTimeAttributes(TArray<FXmlAttribute>& OutAttributes, ITTMLXMLElement* Element, FTTMLParser* Parser, const TArray<FXmlAttribute>& InAttributes);

	void RecalculateTimesFromParent(FTTMLTimeContainer* InParent);

private:
	bool SetTimeContainerType(FTTMLParser* Parser, const FString& InTimeContainerType);

	ETimeType TimeType = ETimeType::Par;
	Electra::FTimeRange Range;
	Electra::FTimeValue Duration;
};





class ITTMLXMLElement : public TSharedFromThis<ITTMLXMLElement, ESPMode::ThreadSafe>
{
public:
	enum class EType
	{
		Other,
		TT,
		Head,
		Body,
		Div,
		P,
		Span,
		AnonymousSpan,
		Br
	};

	ITTMLXMLElement(const FString& InName, EType InType)
		: Name(InName), Type(InType)
	{ }
	virtual ~ITTMLXMLElement() = default;

	virtual EType GetType() const;
	void SetParentElement(const TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>& InParentElement);
	bool PushAndProcessNewElement(FTTMLParser* Parser, const TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>& Element, const TArray<FXmlAttribute>& Attributes, int32 InLineNumber);
	bool SetError(FTTMLParser* Parser, FString ErrorMsg);
	const FString& GetName() const;
	void SetData(const FString& InData);
	void AppendData(const FString& InData);
	const FString& GetData() const;
	void SetLineNumber(int32 InLineNumber);
	int32 GetLineNumber() const;
	TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> GetParentElement() const
	{ return ParentElement.Pin(); }
	virtual bool GetPreserveSpace() const
	{ return bPreserveSpace; }
	virtual void SetPreserveSpace(bool bInPreserveSpace)
	{ bPreserveSpace = bInPreserveSpace; }

	virtual bool ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 LineNumber);
	virtual bool ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& Attributes);
	virtual bool ProcessClose(FTTMLParser* Parser, const FString& ElementName);

	virtual FTTMLTimeContainer* GetTimeContainer()
	{ return nullptr; }
	virtual void RecalculateTimes(bool bRecursively);

protected:
	TWeakPtr<ITTMLXMLElement, ESPMode::ThreadSafe> ParentElement;			//!< Weak pointer to the parent element.
	FString Name;															//!< Element name
	FString Data;															//!< Element data string
	TArray<FXmlAttribute> OtherAttributes;									//!< An array containing attributes not explicitly parsed into element members.
	TArray<TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe>> OtherChildren;	//!< An array of children that were not explicitly parsed.
	EType Type;																//!< Element type
	int32 LineNumber = 0;
	bool bPreserveSpace = false;
};



class FTTMLXML_OtherElement : public ITTMLXMLElement
{
	typedef ITTMLXMLElement Super;
public:
	FTTMLXML_OtherElement(const FString& InName, EType InType=EType::Other) : Super(InName, InType) { }
	virtual ~FTTMLXML_OtherElement() = default;
};


class FTTMLXML_HeadElement : public FTTMLXML_OtherElement
{
	typedef FTTMLXML_OtherElement Super;
public:
	FTTMLXML_HeadElement(const FString& InName, EType InType=EType::Head) : Super(InName, InType) { }
	virtual ~FTTMLXML_HeadElement() = default;
};

// <p> and <span> are basically equivalent. <span> may have @xlink attributes and <p> may have regions,
// both of which we do not support at the moment. Otherwise they are identical and can thus form a
// common class.
class FTTMLXML_ContentElement : public ITTMLXMLElement, public FTTMLTimeContainer
{
	typedef ITTMLXMLElement Super;
public:
	FTTMLXML_ContentElement(const FString& InName, EType InType=EType::Other) : Super(InName, InType) { }
	virtual ~FTTMLXML_ContentElement() = default;
	virtual bool ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 LineNumber) override;
	virtual bool ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& Attributes) override;
	virtual bool ProcessClose(FTTMLParser* Parser, const FString& ElementName) override;

	virtual FTTMLTimeContainer* GetTimeContainer() override
	{ return this; }
	virtual void RecalculateTimes(bool bRecursively);

	virtual void SetText(FString InText);
	virtual const FString& GetText() const
	{ return Text; }

	const TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>>& GetChildren() const
	{ return Children; }

protected:
	void MergeChildren();

	TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> Children;
	FString Text;
};

class FTTMLXML_BR_Element : public FTTMLXML_ContentElement
{
	typedef FTTMLXML_ContentElement Super;
public:
	FTTMLXML_BR_Element(const FString& InName, EType InType=EType::Br) : Super(InName, InType) { }
	virtual ~FTTMLXML_BR_Element() = default;
	virtual const FString& GetText() const
	{ static FString LineFeed(TEXT("\n")); return LineFeed; }
	virtual bool GetPreserveSpace() const
	{ return true; }
	virtual void SetPreserveSpace(bool bInPreserveSpace)
	{ }
};


class FTTMLXML_Span_Element : public FTTMLXML_ContentElement
{
	typedef FTTMLXML_ContentElement Super;
public:
	FTTMLXML_Span_Element(const FString& InName, EType InType=EType::Span) : Super(InName, InType) { }
	virtual ~FTTMLXML_Span_Element() = default;
};


class FTTMLXML_P_Element : public FTTMLXML_ContentElement
{
	typedef FTTMLXML_ContentElement Super;
public:
	FTTMLXML_P_Element(const FString& InName, EType InType=EType::P) : Super(InName, InType) { }
	virtual ~FTTMLXML_P_Element() = default;
};


class FTTMLXML_DivElement : public ITTMLXMLElement, public FTTMLTimeContainer
{
	typedef ITTMLXMLElement Super;
public:
	FTTMLXML_DivElement(const FString& InName, EType InType=EType::Div) : Super(InName, InType) { }
	virtual ~FTTMLXML_DivElement() = default;
	virtual bool ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 LineNumber) override;
	virtual bool ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& Attributes) override;

	virtual FTTMLTimeContainer* GetTimeContainer() override
	{ return this; }
	virtual void RecalculateTimes(bool bRecursively);

	const TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>>& GetContentElements() const
	{ return ContentElements; }

	void CollectContentElementRoots(TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> &OutContentElements, bool bWithPositiveDurationOnly);

private:
	TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> ContentElements;
};


class FTTMLXML_BodyElement : public ITTMLXMLElement, public FTTMLTimeContainer
{
	typedef ITTMLXMLElement Super;
public:
	FTTMLXML_BodyElement(const FString& InName, EType InType=EType::Body) : Super(InName, InType) { }
	virtual ~FTTMLXML_BodyElement() = default;
	virtual bool ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 LineNumber) override;
	virtual bool ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& Attributes) override;

	virtual FTTMLTimeContainer* GetTimeContainer() override
	{ return this; }
	virtual void RecalculateTimes(bool bRecursively);

	const TArray<TSharedPtr<FTTMLXML_DivElement, ESPMode::ThreadSafe>>& GetDivs() const
	{ return Divs; }


	void CollectContentElementRoots(TArray<TSharedPtr<FTTMLXML_ContentElement, ESPMode::ThreadSafe>> &OutContentElements, bool bWithPositiveDurationOnly);

private:
	TArray<TSharedPtr<FTTMLXML_DivElement, ESPMode::ThreadSafe>> Divs;
};


class FTTMLXML_TTElement : public ITTMLXMLElement, public FTTMLTimeContainer
{
	typedef ITTMLXMLElement Super;
public:
	FTTMLXML_TTElement(const FString& InName, EType InType=EType::TT) : Super(InName, InType) { }
	virtual ~FTTMLXML_TTElement() = default;
	virtual bool ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 LineNumber) override;
	virtual bool ProcessAttributes(FTTMLParser* Parser, const TArray<FXmlAttribute>& Attributes) override;

	virtual FTTMLTimeContainer* GetTimeContainer() override
	{ return this; }

	TSharedPtr<FTTMLXML_BodyElement, ESPMode::ThreadSafe> GetBody() const
	{ return Body; }
private:
	TSharedPtr<FTTMLXML_HeadElement, ESPMode::ThreadSafe> Head;
	TSharedPtr<FTTMLXML_BodyElement, ESPMode::ThreadSafe> Body;
};

class FTTMLXML_RootElement : public ITTMLXMLElement
{
	typedef ITTMLXMLElement Super;
public:
	FTTMLXML_RootElement(const FString& InName, EType InType=EType::Other) : Super(InName, InType) { }
	virtual ~FTTMLXML_RootElement() = default;
	virtual bool ProcessElement(FTTMLParser* Parser, const FString& ElementName, const TArray<FXmlAttribute>& Attributes, int32 LineNumber) override;

	TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> GetTT() const
	{ return TT; }

private:
	TSharedPtr<ITTMLXMLElement, ESPMode::ThreadSafe> TT;
};

}
