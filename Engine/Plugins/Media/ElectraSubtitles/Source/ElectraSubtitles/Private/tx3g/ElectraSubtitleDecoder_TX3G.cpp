// Copyright Epic Games, Inc. All Rights Reserved.

#include "tx3g/ElectraSubtitleDecoder_TX3G.h"
#include "ElectraSubtitleDecoderFactory.h"
#include "ElectraSubtitleModule.h"
#include "MediaDecoderOutput.h"

namespace ElectraSubtitleDecoderTX3GUtils
{

namespace OptionKeys
{
	static const FName Width(TEXT("width"));
	static const FName Height(TEXT("height"));
	static const FName OffsetX(TEXT("offset_x"));
	static const FName OffsetY(TEXT("offset_y"));
	static const FName Timescale(TEXT("timescale"));
}

#if !PLATFORM_LITTLE_ENDIAN
	static inline uint8  GetFromBigEndian(uint8 value)		{ return value; }
	static inline int8   GetFromBigEndian(int8 value)		{ return value; }
	static inline uint16 GetFromBigEndian(uint16 value)		{ return value; }
	static inline int16 GetFromBigEndian(int16 value)		{ return value; }
	static inline int32 GetFromBigEndian(int32 value)		{ return value; }
	static inline uint32 GetFromBigEndian(uint32 value)		{ return value; }
	static inline int64 GetFromBigEndian(int64 value)		{ return value; }
	static inline uint64 GetFromBigEndian(uint64 value)		{ return value; }
#else
	static inline uint16 EndianSwap(uint16 value)			{ return (value >> 8) | (value << 8); }
	static inline int16 EndianSwap(int16 value)				{ return int16(EndianSwap(uint16(value))); }
	static inline uint32 EndianSwap(uint32 value)			{ return (value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24); }
	static inline int32 EndianSwap(int32 value)				{ return int32(EndianSwap(uint32(value))); }
	static inline uint64 EndianSwap(uint64 value)			{ return (uint64(EndianSwap(uint32(value & 0xffffffffU))) << 32) | uint64(EndianSwap(uint32(value >> 32))); }
	static inline int64 EndianSwap(int64 value)				{ return int64(EndianSwap(uint64(value)));}
	static inline uint8  GetFromBigEndian(uint8 value)		{ return value; }
	static inline int8   GetFromBigEndian(int8 value)		{ return value; }
	static inline uint16 GetFromBigEndian(uint16 value)		{ return EndianSwap(value); }
	static inline int16 GetFromBigEndian(int16 value)		{ return EndianSwap(value); }
	static inline int32 GetFromBigEndian(int32 value)		{ return EndianSwap(value); }
	static inline uint32 GetFromBigEndian(uint32 value)		{ return EndianSwap(value); }
	static inline int64 GetFromBigEndian(int64 value)		{ return EndianSwap(value); }
	static inline uint64 GetFromBigEndian(uint64 value)		{ return EndianSwap(value); }
#endif

	template <typename T>
	T ValueFromBigEndian(const T value)
	{
		return GetFromBigEndian(value);
	}


	class FDataReaderMP4
	{
	public:
		FDataReaderMP4(const TArray<uint8>& InDataBufferToReadFrom) : DataBufferRef(InDataBufferToReadFrom)
		{}

		int32 GetCurrentOffset() const
		{ return CurrentOffset;	}

		int32 GetNumBytesRemaining() const
		{ return DataBufferRef.Num() - GetCurrentOffset(); }

		template <typename T>
		bool Read(T& value)
		{
			T Temp = 0;
			int64 NumRead = ReadData(&Temp, sizeof(T));
			if (NumRead == sizeof(T))
			{
				value = ElectraSubtitleDecoderTX3GUtils::ValueFromBigEndian(Temp);
				return true;
			}
			return false;
		}

		bool ReadString(FString& OutString, uint16 NumBytes)
		{
			OutString.Empty();
			if (NumBytes == 0)
			{
				return true;
			}
			TArray<uint8> Buf;
			Buf.AddUninitialized(NumBytes);
			if (ReadBytes(Buf.GetData(), NumBytes))
			{
				// Check for UTF16 BOM
				if (NumBytes >= 2 && ((Buf[0] == 0xff && Buf[1] == 0xfe) || (Buf[0] == 0xfe && Buf[1] == 0xff)))
				{
					UE_LOGF(LogElectraSubtitles, Error, "TX3G uses UTF16 which is not supported");
					return false;
				}
				FUTF8ToTCHAR cnv((const ANSICHAR*)Buf.GetData(), NumBytes);
				OutString = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
				return true;
			}
			return false;
		}

		bool ReadBytes(void* Buffer, int32 NumBytes)
		{
			int32 NumRead = ReadData(Buffer, NumBytes);
			return NumRead == NumBytes;
		}

#define MAKE_BOX_ATOM(a,b,c,d) (uint32)((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d)
		static const uint32 BoxType_styl = MAKE_BOX_ATOM('s', 't', 'y', 'l');
		static const uint32 BoxType_hlit = MAKE_BOX_ATOM('h', 'l', 'i', 't');
		static const uint32 BoxType_hclr = MAKE_BOX_ATOM('h', 'c', 'l', 'r');
		static const uint32 BoxType_krok = MAKE_BOX_ATOM('k', 'r', 'o', 'k');
		static const uint32 BoxType_dlay = MAKE_BOX_ATOM('d', 'l', 'a', 'y');
		static const uint32 BoxType_href = MAKE_BOX_ATOM('h', 'r', 'e', 'f');
		static const uint32 BoxType_tbox = MAKE_BOX_ATOM('t', 'b', 'o', 'x');
		static const uint32 BoxType_blnk = MAKE_BOX_ATOM('b', 'l', 'n', 'k');
		static const uint32 BoxType_twrp = MAKE_BOX_ATOM('t', 'w', 'r', 'p');
#undef MAKE_BOX_ATOM
	private:
		int32 ReadData(void* IntoBuffer, int32 NumBytesToRead)
		{
			if (NumBytesToRead == 0)
			{
				return 0;
			}
			int32 NumAvail = DataBufferRef.Num() - CurrentOffset;
			if (NumAvail >= NumBytesToRead)
			{
				if (IntoBuffer)
				{
					FMemory::Memcpy(IntoBuffer, DataBufferRef.GetData() + CurrentOffset, NumBytesToRead);
				}
				CurrentOffset += NumBytesToRead;
				return NumBytesToRead;
			}
			return -1;
		}

		const TArray<uint8>& DataBufferRef;
		int32 CurrentOffset = 0;
	};

}



class FElectraSubtitleDecoderFactoryTX3G : public IElectraSubtitleDecoderFactory
{
public:
	virtual TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> CreateDecoder(const FString& SubtitleCodecName) override;
};


void FElectraSubtitleDecoderTX3G::RegisterCodecs(IElectraSubtitleDecoderFactoryRegistry& InRegistry)
{
	static FElectraSubtitleDecoderFactoryTX3G Factory;
	TArray<IElectraSubtitleDecoderFactoryRegistry::FCodecInfo> CodecInfos { { FString(TEXT("tx3g")), 0} };
	InRegistry.AddDecoderFactory(CodecInfos, &Factory);
}


TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> FElectraSubtitleDecoderFactoryTX3G::CreateDecoder(const FString& SubtitleCodecName)
{
	return MakeShared<FElectraSubtitleDecoderTX3G, ESPMode::ThreadSafe>();
}






class FSubtitleDecoderOutputTX3G : public ISubtitleDecoderOutput
{
public:
	virtual ~FSubtitleDecoderOutputTX3G() = default;

	void SetText(const FString& InText)
	{
		FTCHARToUTF8 Converted(*InText); // Convert to UTF8
		TextAsArray.Empty();
		TextAsArray.Append(reinterpret_cast<const uint8*>(Converted.Get()), Converted.Length());
	}

	void SetDuration(const Electra::FTimeValue& InDuration)
	{
		Duration = InDuration.GetAsTimespan();
	}

	void SetTimestamp(const Electra::FTimeValue& InTimestamp)
	{
		Timestamp.Time = InTimestamp.GetAsTimespan();
		Timestamp.SequenceIndex = InTimestamp.GetSequenceIndex();
	}

	void SetID(const FString& InID)
	{
		ID = InID;
	}


	const TArray<uint8>& GetData() override
	{
		return TextAsArray;
	}

	FDecoderTimeStamp GetTime() const override
	{
		return Timestamp;
	}

	void SetTime(FDecoderTimeStamp& InTime) override
	{
		Timestamp = InTime;
	}

	FTimespan GetDuration() const override
	{
		return Duration;
	}

	const FString& GetFormat() const override
	{
		static FString Format(TEXT("tx3g"));
		return Format;
	}
	const FString& GetID() const override
	{
		return ID;
	}
private:
	TArray<uint8> TextAsArray;
	FString ID;
	FDecoderTimeStamp Timestamp;
	FTimespan Duration;
};




FElectraSubtitleDecoderTX3G::FElectraSubtitleDecoderTX3G()
{
}

FElectraSubtitleDecoderTX3G::~FElectraSubtitleDecoderTX3G()
{
}

bool FElectraSubtitleDecoderTX3G::InitializeStreamWithCSD(const TArray<uint8>& InCSD, const Electra::FParamDict& InAdditionalInfo)
{
	#define RETURN_IF_ERROR(expr)								\
		if (!expr)												\
		{														\
			UE_LOGF(LogElectraSubtitles, Error, "Not enough CSD bytes to parse TX3G text sample entry");	\
			return false;										\
		}														\


	// Get dimension, placement offset and other data from the sideband dictionary.
	Width = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTX3GUtils::OptionKeys::Width).SafeGetInt64(0);
	Height = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTX3GUtils::OptionKeys::Height).SafeGetInt64(0);
	TranslationX = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTX3GUtils::OptionKeys::OffsetX).SafeGetInt64(0);
	TranslationY = (int32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTX3GUtils::OptionKeys::OffsetY).SafeGetInt64(0);
	Timescale = (uint32) InAdditionalInfo.GetValue(ElectraSubtitleDecoderTX3GUtils::OptionKeys::Timescale).SafeGetInt64(0);

	ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4 r(InCSD);

	// The CSD is a PlainTextSampleEntry of an mp4 (ISO/IEC 14496-12) file and interpreted as per
	// ETSI TS 126 245 V11.0.0 - 5.16 Sample Description Format

	RETURN_IF_ERROR(r.ReadBytes(nullptr, 6));
	RETURN_IF_ERROR(r.Read(DataReferenceIndex));

	RETURN_IF_ERROR(r.Read(DisplayFlags));
	RETURN_IF_ERROR(r.Read(HorizontalJustification));
	RETURN_IF_ERROR(r.Read(VerticalJustification));
	RETURN_IF_ERROR(r.ReadBytes(BackgroundColorRGBA, 4));

	// BoxRecord
	RETURN_IF_ERROR(r.Read(BoxRecord.Top));
	RETURN_IF_ERROR(r.Read(BoxRecord.Left));
	RETURN_IF_ERROR(r.Read(BoxRecord.Bottom));
	RETURN_IF_ERROR(r.Read(BoxRecord.Right));

	// Style record
	RETURN_IF_ERROR(r.Read(StyleRecord.StartChar));
	RETURN_IF_ERROR(r.Read(StyleRecord.EndChar));
	RETURN_IF_ERROR(r.Read(StyleRecord.FontID));
	RETURN_IF_ERROR(r.Read(StyleRecord.FaceStyleFlags));
	RETURN_IF_ERROR(r.Read(StyleRecord.FontSize));
	RETURN_IF_ERROR(r.ReadBytes(StyleRecord.TextColorRGBA, 4));

	// The font table is stored in a box. We skip over the box itself and parse it in place.
	// Any boxes besides a single ftab box we ignore.
	uint8 TempFTAB[8];
	RETURN_IF_ERROR(r.ReadBytes(TempFTAB, 8));
	if (TempFTAB[4] != 'f' || TempFTAB[5] != 't' || TempFTAB[6] != 'a' || TempFTAB[7] != 'b')
	{
		UE_LOGF(LogElectraSubtitles, Error, "TX3G CSD is missing the ftab box!");
		return false;
	}

	// FontTableBox
	uint16 EntryCount;
	RETURN_IF_ERROR(r.Read(EntryCount));
	FontRecords.Reserve(EntryCount);
	for(int32 nEntry=0; nEntry<EntryCount; ++nEntry)
	{
		FFontRecord& fr = FontRecords.AddDefaulted_GetRef();
		RETURN_IF_ERROR(r.Read(fr.FontID));
		uint8 FontNameLength;
		RETURN_IF_ERROR(r.Read(FontNameLength));
		if (FontNameLength)
		{
			if (!r.ReadString(fr.FontName, FontNameLength))
			{
				return false;
			}
		}
	}

	#undef RETURN_IF_ERROR
	return true;
}

IElectraSubtitleDecoder::FOnSubtitleReceivedDelegate& FElectraSubtitleDecoderTX3G::GetParsedSubtitleReceiveDelegate()
{
	return ParsedSubtitleDelegate;
}

Electra::FTimeValue FElectraSubtitleDecoderTX3G::GetStreamedDeliveryTimeOffset()
{
	return Electra::FTimeValue::GetZero();
}

void FElectraSubtitleDecoderTX3G::AddStreamedSubtitleData(const TArray<uint8>& InData, Electra::FTimeValue InAbsoluteTimestamp, Electra::FTimeValue InDuration, const Electra::FParamDict& InAdditionalInfo)
{
	#define RETURN_IF_ERROR(expr)															\
		if (!expr)																			\
		{																					\
			UE_LOGF(LogElectraSubtitles, Error, "Bad TX3G text sample, ignoring.");	\
			return;																			\
		}																					\

	ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4 r(InData);

	// Start a subtitle entry.
	FSubtitleEntry Sub;
	// By default it uses the same box area as defined in the CSD. Each subtitle can override this locally.
	Sub.Box = BoxRecord;

	uint16 TextLen;
	RETURN_IF_ERROR(r.Read(TextLen));
	if (TextLen)
	{
		if (r.ReadString(Sub.Text, TextLen))
		{
			// Get the number of trailing bytes that describe styling boxes.
			while(r.GetNumBytesRemaining() > 0)
			{
				uint32 BoxLen, BoxType;
				if (!r.Read(BoxLen) || !r.Read(BoxType))
				{
					UE_LOGF(LogElectraSubtitles, Error, "Bad TX3G text sample formatting box, ignoring.");
					break;
				}
				if (BoxLen < 8 || (int32)(BoxLen-8 )> r.GetNumBytesRemaining())
				{
					UE_LOGF(LogElectraSubtitles, Error, "Bad TX3G text sample formatting box, ignoring.");
					break;
				}
				if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_styl)
				{
					uint16 NumStyleRecords;
					RETURN_IF_ERROR(r.Read(NumStyleRecords));
					for(int32 i=0; i<NumStyleRecords; ++i)
					{
						FStyleRecord sr;
						RETURN_IF_ERROR(r.Read(sr.StartChar));
						RETURN_IF_ERROR(r.Read(sr.EndChar));
						RETURN_IF_ERROR(r.Read(sr.FontID));
						RETURN_IF_ERROR(r.Read(sr.FaceStyleFlags));
						RETURN_IF_ERROR(r.Read(sr.FontSize));
						RETURN_IF_ERROR(r.ReadBytes(sr.TextColorRGBA, 4));
						Sub.Styles.Emplace(MoveTemp(sr));
					}
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_hlit)
				{
					FHighlight hl;
					RETURN_IF_ERROR(r.Read(hl.StartCharOffset));
					RETURN_IF_ERROR(r.Read(hl.EndCharOffset));
					Sub.Highlights.Emplace(MoveTemp(hl));
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_hclr)
				{
					FHighlightColor hc;
					RETURN_IF_ERROR(r.ReadBytes(hc.TextColorRGBA, 4));
					Sub.HighlightColor.Emplace(MoveTemp(hc));
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_krok)
				{
					FTextKaraoke kr;
					RETURN_IF_ERROR(r.Read(kr.HighlightStartTime));
					uint16 NumEntries;
					RETURN_IF_ERROR(r.Read(NumEntries));
					kr.Entries.Reserve(NumEntries);
					for(int32 i=0; i<NumEntries; ++i)
					{
						FTextKaraoke::FEntry& e=kr.Entries.AddDefaulted_GetRef();
						RETURN_IF_ERROR(r.Read(e.HightlightEndTime));
						RETURN_IF_ERROR(r.Read(e.StartCharOffset));
						RETURN_IF_ERROR(r.Read(e.EndCharOffset));
					}
					Sub.Karaoke.Emplace(MoveTemp(kr));
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_dlay)
				{
					RETURN_IF_ERROR(r.Read(Sub.ScrollDelay.ScrollDelay));
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_href)
				{
					uint8 Len;
					FHyperText ht;
					RETURN_IF_ERROR(r.Read(ht.StartCharOffset));
					RETURN_IF_ERROR(r.Read(ht.EndCharOffset));
					RETURN_IF_ERROR(r.Read(Len));
					RETURN_IF_ERROR(r.ReadString(ht.URL, Len));
					RETURN_IF_ERROR(r.Read(Len));
					RETURN_IF_ERROR(r.ReadString(ht.AltString, Len));
					Sub.HyperTexts.Emplace(MoveTemp(ht));
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_tbox)
				{
					RETURN_IF_ERROR(r.Read(Sub.Box.Top));
					RETURN_IF_ERROR(r.Read(Sub.Box.Left));
					RETURN_IF_ERROR(r.Read(Sub.Box.Bottom));
					RETURN_IF_ERROR(r.Read(Sub.Box.Right));
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_blnk)
				{
					FBlink bk;
					RETURN_IF_ERROR(r.Read(bk.StartCharOffset));
					RETURN_IF_ERROR(r.Read(bk.EndCharOffset));
					Sub.Blinks.Emplace(MoveTemp(bk));
				}
				else if (BoxType == ElectraSubtitleDecoderTX3GUtils::FDataReaderMP4::BoxType_twrp)
				{
					FTextWrap wr;
					RETURN_IF_ERROR(r.Read(wr.WrapFlag));
					Sub.TextWrap.Emplace(MoveTemp(wr));
				}
				else
				{
					if (!r.ReadBytes(nullptr, BoxLen - 8))
					{
						break;
					}
				}
			}
		}
	}
	else
	{
		// An empty entry.
	}

	TSharedPtr<FSubtitleDecoderOutputTX3G, ESPMode::ThreadSafe> Out = MakeShared<FSubtitleDecoderOutputTX3G, ESPMode::ThreadSafe>();
	Out->SetTimestamp(InAbsoluteTimestamp);
	Out->SetDuration(InDuration);
	Out->SetText(Sub.Text);
	Out->SetID(LexToString(++NextID));
	ParsedSubtitleDelegate.Broadcast(Out);

	#undef RETURN_IF_ERROR
}

void FElectraSubtitleDecoderTX3G::SignalStreamedSubtitleEOD()
{
}

void FElectraSubtitleDecoderTX3G::Flush()
{
}

void FElectraSubtitleDecoderTX3G::Start()
{
}

void FElectraSubtitleDecoderTX3G::Stop()
{
}

void FElectraSubtitleDecoderTX3G::UpdatePlaybackPosition(Electra::FTimeValue InAbsolutePosition, Electra::FTimeValue InLocalPosition)
{
}

