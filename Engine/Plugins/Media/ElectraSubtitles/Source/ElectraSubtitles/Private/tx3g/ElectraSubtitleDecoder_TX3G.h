// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraSubtitleDecoder.h"

class IElectraSubtitleDecoderFactoryRegistry;

/**
 * 3GPP / TX3G subtitle decoder (ETSI TS 126 245 V11.0.0)
 */
class FElectraSubtitleDecoderTX3G : public IElectraSubtitleDecoder
{
public:
	static void RegisterCodecs(IElectraSubtitleDecoderFactoryRegistry& InRegistry);

	FElectraSubtitleDecoderTX3G();

	virtual ~FElectraSubtitleDecoderTX3G();

	//-------------------------------------------------------------------------
	// Methods from IElectraSubtitleDecoder
	//
	virtual bool InitializeStreamWithCSD(const TArray<uint8>& InCSD, const Electra::FParamDict& InAdditionalInfo) override;
	virtual IElectraSubtitleDecoder::FOnSubtitleReceivedDelegate& GetParsedSubtitleReceiveDelegate() override;
	virtual Electra::FTimeValue GetStreamedDeliveryTimeOffset() override;
	virtual void AddStreamedSubtitleData(const TArray<uint8>& InData, Electra::FTimeValue InAbsoluteTimestamp, Electra::FTimeValue InDuration, const Electra::FParamDict& InAdditionalInfo) override;
	virtual void SignalStreamedSubtitleEOD() override;
	virtual void Flush() override;
	virtual void Start() override;
	virtual void Stop() override;
	virtual void UpdatePlaybackPosition(Electra::FTimeValue InAbsolutePosition, Electra::FTimeValue InLocalPosition) override;

private:
	struct FBoxRecord
	{
		int16 Top = 0;
		int16 Left = 0;
		int16 Bottom = 0;
		int16 Right = 0;
	};

	struct FStyleRecord
	{
		uint16 StartChar = 0;
		uint16 EndChar = 0;
		uint16 FontID = 0;
		uint8 FaceStyleFlags = 0;
		uint8 FontSize = 0;
		uint8 TextColorRGBA[4] { 0 };
	};

	struct FFontRecord
	{
		uint16 FontID = 0;
		FString FontName;
	};

	struct FHighlight
	{
		uint16 StartCharOffset = 0;
		uint16 EndCharOffset = 0;
	};

	struct FHighlightColor
	{
		uint8 TextColorRGBA[4] { 0 };
	};

	struct FTextKaraoke
	{
		uint32 HighlightStartTime = 0;
		struct FEntry
		{
			uint32 HightlightEndTime = 0;
			uint16 StartCharOffset = 0;
			uint16 EndCharOffset = 0;
		};
		TArray<FEntry> Entries;
	};

	struct FScrollDelay
	{
		uint32 ScrollDelay = 0;
	};

	struct FHyperText
	{
		uint16 StartCharOffset = 0;
		uint16 EndCharOffset = 0;
		FString URL;
		FString AltString;
	};

	struct FBlink
	{
		uint16 StartCharOffset = 0;
		uint16 EndCharOffset = 0;
	};

	struct FTextWrap
	{
		uint8 WrapFlag = 0;
	};


	struct FSubtitleEntry
	{
		FString						Text;
		FBoxRecord					Box;
		TArray<FStyleRecord>		Styles;
		TArray<FHighlight>			Highlights;
		TOptional<FHighlightColor>	HighlightColor;
		TOptional<FTextKaraoke>		Karaoke;
		FScrollDelay				ScrollDelay;
		TArray<FHyperText>			HyperTexts;
		TArray<FBlink>				Blinks;
		TOptional<FTextWrap>		TextWrap;
	};

	FOnSubtitleReceivedDelegate ParsedSubtitleDelegate;
	uint32 NextID = 0;

	uint16				DataReferenceIndex = 0;
	int32				Width = 0;
	int32				Height = 0;
	int32				TranslationX = 0;
	int32				TranslationY = 0;
	uint32				Timescale = 0;
	uint32				DisplayFlags = 0;
	int8				HorizontalJustification = 0;
	int8				VerticalJustification = 0;
	uint8				BackgroundColorRGBA[4] {0};
	FBoxRecord			BoxRecord;
	FStyleRecord		StyleRecord;
	TArray<FFontRecord>	FontRecords;
};
