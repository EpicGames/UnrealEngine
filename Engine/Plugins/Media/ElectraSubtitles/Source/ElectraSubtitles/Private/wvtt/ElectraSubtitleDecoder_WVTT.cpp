// Copyright Epic Games, Inc. All Rights Reserved.

#include "wvtt/ElectraSubtitleDecoder_WVTT.h"
#include "wvtt/WebVTTParser.h"
#include "ElectraSubtitleDecoderFactory.h"
#include "ElectraSubtitleModule.h"
#include "MediaDecoderOutput.h"

namespace ElectraSubtitleDecoderWVTTUtils
{

namespace OptionKeys
{
	static const FName Width(TEXT("width"));
	static const FName Height(TEXT("height"));
	static const FName OffsetX(TEXT("offset_x"));
	static const FName OffsetY(TEXT("offset_y"));
	static const FName Timescale(TEXT("timescale"));
	static const FName SourceID(TEXT("source_id"));
	static const FName SendEmptySubtitleDuringGaps(TEXT("sendEmptySubtitleDuringGaps"));
}

#if !PLATFORM_LITTLE_ENDIAN
	static inline uint8 GetFromBigEndian(uint8 value)		{ return value; }
	static inline int8 GetFromBigEndian(int8 value)			{ return value; }
	static inline uint16 GetFromBigEndian(uint16 value)		{ return value; }
	static inline int16 GetFromBigEndian(int16 value)		{ return value; }
	static inline int32 GetFromBigEndian(int32 value)		{ return value; }
	static inline uint32 GetFromBigEndian(uint32 value)		{ return value; }
	static inline int64 GetFromBigEndian(int64 value)		{ return value; }
	static inline uint64 GetFromBigEndian(uint64 value)		{ return value; }
#else
	static inline uint16 EndianSwap(uint16 value)			{ return (value >> 8) | (value << 8); }
	static inline int16 EndianSwap(int16 value)				{ return int16(EndianSwap(uint16(value))); }
	static inline uint32 EndianSwap(uint32 value)			{ return (value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24); }
	static inline int32 EndianSwap(int32 value)				{ return int32(EndianSwap(uint32(value))); }
	static inline uint64 EndianSwap(uint64 value)			{ return (uint64(EndianSwap(uint32(value & 0xffffffffU))) << 32) | uint64(EndianSwap(uint32(value >> 32))); }
	static inline int64 EndianSwap(int64 value)				{ return int64(EndianSwap(uint64(value)));}
	static inline uint8 GetFromBigEndian(uint8 value)		{ return value; }
	static inline int8 GetFromBigEndian(int8 value)			{ return value; }
	static inline uint16 GetFromBigEndian(uint16 value)		{ return EndianSwap(value); }
	static inline int16 GetFromBigEndian(int16 value)		{ return EndianSwap(value); }
	static inline int32 GetFromBigEndian(int32 value)		{ return EndianSwap(value); }
	static inline uint32 GetFromBigEndian(uint32 value)		{ return EndianSwap(value); }
	static inline int64 GetFromBigEndian(int64 value)		{ return EndianSwap(value); }
	static inline uint64 GetFromBigEndian(uint64 value)		{ return EndianSwap(value); }
#endif

	template <typename T>
	T ValueFromBigEndian(const T value)
	{
		return GetFromBigEndian(value);
	}


	class FDataReaderMP4
	{
	public:
		FDataReaderMP4(const TArray<uint8>& InDataBufferToReadFrom) : DataBufferRef(InDataBufferToReadFrom)
		{}

		int32 GetCurrentOffset() const
		{ return CurrentOffset;	}

		int32 GetNumBytesRemaining() const
		{ return DataBufferRef.Num() - GetCurrentOffset(); }

		template <typename T>
		bool Read(T& value)
		{
			T Temp = 0;
			int64 NumRead = ReadData(&Temp, sizeof(T));
			if (NumRead == sizeof(T))
			{
				value = ElectraSubtitleDecoderWVTTUtils::ValueFromBigEndian(Temp);
				return true;
			}
			return false;
		}

		bool ReadString(FString& OutString, uint16 NumBytes)
		{
			OutString.Empty();
			if (NumBytes == 0)
			{
				return true;
			}
			TArray<uint8> Buf;
			Buf.AddUninitialized(NumBytes);
			if (ReadBytes(Buf.GetData(), NumBytes))
			{
				// Check for UTF16 BOM
				if (NumBytes >= 2 && ((Buf[0] == 0xff && Buf[1] == 0xfe) || (Buf[0] == 0xfe && Buf[1] == 0xff)))
				{
					UE_LOGF(LogElectraSubtitles, Error, "WVTT uses UTF16 which is not supported");
					return false;
				}
				FUTF8ToTCHAR cnv((const ANSICHAR*)Buf.GetData(), NumBytes);
				OutString = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
				return true;
			}
			return false;
		}

		bool ReadBytes(void* Buffer, int32 NumBytes)
		{
			int32 NumRead = ReadData(Buffer, NumBytes);
			return NumRead == NumBytes;
		}

#define MAKE_BOX_ATOM(a,b,c,d) (uint32)((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d)
		static const uint32 BoxType_vttC = MAKE_BOX_ATOM('v', 't', 't', 'C');
		static const uint32 BoxType_vlab = MAKE_BOX_ATOM('v', 'l', 'a', 'b');
		static const uint32 BoxType_vtte = MAKE_BOX_ATOM('v', 't', 't', 'e');
		static const uint32 BoxType_vtta = MAKE_BOX_ATOM('v', 't', 't', 'a');
		static const uint32 BoxType_vttc = MAKE_BOX_ATOM('v', 't', 't', 'c');
		static const uint32 BoxType_vsid = MAKE_BOX_ATOM('v', 's', 'i', 'd');
		static const uint32 BoxType_ctim = MAKE_BOX_ATOM('c', 't', 'i', 'm');
		static const uint32 BoxType_iden = MAKE_BOX_ATOM('i', 'd', 'e', 'n');
		static const uint32 BoxType_sttg = MAKE_BOX_ATOM('s', 't', 't', 'g');
		static const uint32 BoxType_payl = MAKE_BOX_ATOM('p', 'a', 'y', 'l');
#undef MAKE_BOX_ATOM
	private:
		int32 ReadData(void* IntoBuffer, int32 NumBytesToRead)
		{
			if (NumBytesToRead == 0)
			{
				return 0;
			}
			int32 NumAvail = DataBufferRef.Num() - CurrentOffset;
			if (NumAvail >= NumBytesToRead)
			{
				if (IntoBuffer)
				{
					FMemory::Memcpy(IntoBuffer, DataBufferRef.GetData() + CurrentOffset, NumBytesToRead);
				}
				CurrentOffset += NumBytesToRead;
				return NumBytesToRead;
			}
			return -1;
		}

		const TArray<uint8>& DataBufferRef;
		int32 CurrentOffset = 0;
	};

}



class FElectraSubtitleDecoderFactoryWVTT : public IElectraSubtitleDecoderFactory
{
public:
	virtual TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> CreateDecoder(const FString& SubtitleCodecName) override;
};


void FElectraSubtitleDecoderWVTT::RegisterCodecs(IElectraSubtitleDecoderFactoryRegistry& InRegistry)
{
	static FElectraSubtitleDecoderFactoryWVTT Factory;
	TArray<IElectraSubtitleDecoderFactoryRegistry::FCodecInfo> CodecInfos
	{
		{ FString(TEXT("wvtt")), 0},			// codec
		{ FString(TEXT("text/vtt")), 0}			// mimetype
	};
	InRegistry.AddDecoderFactory(CodecInfos, &Factory);
}


TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> FElectraSubtitleDecoderFactoryWVTT::CreateDecoder(const FString& SubtitleCodecName)
{
	return MakeShared<FElectraSubtitleDecoderWVTT, ESPMode::ThreadSafe>();
}






class FSubtitleDecoderOutputWVTT : public ISubtitleDecoderOutput
{
public:
	virtual ~FSubtitleDecoderOutputWVTT() = default;

	void SetText(const FString& InText)
	{
		FTCHARToUTF8 Converted(*InText); // Convert to UTF8
		TextAsArray.Empty();
		TextAsArray.Append(reinterpret_cast<const uint8*>(Converted.Get()), Converted.Length());
	}

	void SetDuration(const Electra::FTimeValue& InDuration)
	{
		Duration = InDuration.GetAsTimespan();
	}

	void SetTimestamp(const Electra::FTimeValue& InTimestamp)
	{
		Timestamp.Time = InTimestamp.GetAsTimespan();
		Timestamp.SequenceIndex = InTimestamp.GetSequenceIndex();
	}

	void SetID(const FString& InID, bool bInIsGeneratedID)
	{
		ID = InID;
		bIsGeneratedID = bInIsGeneratedID;
	}

	const TArray<uint8>& GetData() override
	{
		return TextAsArray;
	}

	FDecoderTimeStamp GetTime() const override
	{
		return Timestamp;
	}

	void SetTime(FDecoderTimeStamp& InTime) override
	{
		Timestamp = InTime;
	}

	FTimespan GetDuration() const override
	{
		return Duration;
	}

	const FString& GetFormat() const override
	{
		static FString Format(TEXT("wvtt"));
		return Format;
	}
	const FString& GetID() const override
	{
		return ID;
	}

	bool operator == (const FSubtitleDecoderOutputWVTT& Other) const
	{
		return Timestamp.Time == Other.Timestamp.Time
			&& (bIsGeneratedID || Other.bIsGeneratedID || ID.Equals(Other.ID))
			&& Duration == Other.Duration && TextAsArray == Other.TextAsArray;
	}

	bool operator != (const FSubtitleDecoderOutputWVTT& Other) const
	{
		return !(this->operator==(Other));
	}
private:
	TArray<uint8> TextAsArray;
	FString ID;
	FDecoderTimeStamp Timestamp;
	FTimespan Duration;
	bool bIsGeneratedID = false;
};




FElectraSubtitleDecoderWVTT::FElectraSubtitleDecoderWVTT()
{
}

FElectraSubtitleDecoderWVTT::~FElectraSubtitleDecoderWVTT()
{
}

bool FElectraSubtitleDecoderWVTT::InitializeStreamWithCSD(const TArray<uint8>& InCSD, const Electra::FParamDict& InAdditionalInfo)
{
	// Get data from the sideband dictionary.
	bool bIsSideloaded = InCSD.IsEmpty();
	// Check if the CSD starts with "WEBVTT" or BOM+"WEBVTT"
	bool bIsRaw = (InCSD.Num() >= 6 && InCSD[0] == 0x57 && InCSD[1] == 0x45 && InCSD[2] == 0x42 && InCSD[3] == 0x56 && InCSD[4] == 0x54 && InCSD[5] == 0x54) ||
				  (InCSD.Num() >= 9 && InCSD[0] == 0xef && InCSD[1] == 0xbb && InCSD[2] == 0xbf && InCSD[3] == 0x57 && InCSD[4] == 0x45 && InCSD[5] == 0x42 && InCSD[6] == 0x56 && InCSD[7] == 0x54 && InCSD[8] == 0x54);
	if (bIsSideloaded || bIsRaw)
	{
		bNeedsParsing = true;
	}
	else
	{
		#define RETURN_IF_ERROR(expr)																				\
			if (!expr)																								\
			{																										\
				UE_LOGF(LogElectraSubtitles, Error, "Not enough CSD bytes to parse WVTT text sample entry");	\
				return false;																						\
			}
		ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4 r(InCSD);
		FString Configuration;
		FString Label;

		// The CSD is a WVTTSampleEntry of an mp4 (ISO/IEC 14496-12) file and interpreted as per
		// ISO/IEC 14496-30 Section 7.5 Sample entry format
		RETURN_IF_ERROR(r.ReadBytes(nullptr, 6));
		uint16 DataReferenceIndex = 0;
		RETURN_IF_ERROR(r.Read(DataReferenceIndex));

		// Read the configuration boxes.
		// There needs to be a 'vttC' box.
		// 'vlab' and 'btrt' boxes are optional. Anything else is ignored.
		while(r.GetNumBytesRemaining() > 0)
		{
			uint32 BoxLen, BoxType;
			if (!r.Read(BoxLen) || !r.Read(BoxType))
			{
				UE_LOGF(LogElectraSubtitles, Error, "Bad WVTT box in CSD, ignoring.");
				break;
			}
			if (BoxLen < 8 || (int32)(BoxLen-8 )> r.GetNumBytesRemaining())
			{
				UE_LOGF(LogElectraSubtitles, Error, "Bad WVTT box in CSD, ignoring.");
				break;
			}

			if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_vttC)
			{
				RETURN_IF_ERROR(r.ReadString(Configuration, BoxLen - 8));
			}
			else if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_vlab)
			{
				RETURN_IF_ERROR(r.ReadString(Label, BoxLen - 8));
			}
			else
			{
				// Note: We ignore a potential 'btrt' box here. There is no use for us.
				if (!r.ReadBytes(nullptr, BoxLen - 8))
				{
					break;
				}
			}
		}

		// The "configuration" is everything up to the first cue, so at least the string "WEBVTT".
		if (Configuration.IsEmpty() || !Configuration.StartsWith(TEXT("WEBVTT")))
		{
			UE_LOGF(LogElectraSubtitles, Error, "Bad WVTT configuration box 'vttC'!");
			return false;
		}
		#undef RETURN_IF_ERROR
	}

	bSendEmptySubtitleDuringGaps = InAdditionalInfo.GetValue(ElectraSubtitleDecoderWVTTUtils::OptionKeys::SendEmptySubtitleDuringGaps).SafeGetBool();
	return true;
}

IElectraSubtitleDecoder::FOnSubtitleReceivedDelegate& FElectraSubtitleDecoderWVTT::GetParsedSubtitleReceiveDelegate()
{
	return ParsedSubtitleDelegate;
}

Electra::FTimeValue FElectraSubtitleDecoderWVTT::GetStreamedDeliveryTimeOffset()
{
	return Electra::FTimeValue::GetZero();
}

void FElectraSubtitleDecoderWVTT::AddStreamedSubtitleData(const TArray<uint8>& InData, Electra::FTimeValue InAbsoluteTimestamp, Electra::FTimeValue InDuration, const Electra::FParamDict& InAdditionalInfo)
{
	if (bNeedsParsing)
	{
		// Check if we already have this.
		FScopeLock lock(&ParsedTimerangeLock);
		for(int32 i=0; i<ParsedTimeranges.Num(); ++i)
		{
			if (ParsedTimeranges[i]->SourceID.Equals(InAdditionalInfo.GetValue(ElectraSubtitleDecoderWVTTUtils::OptionKeys::SourceID).SafeGetFString()) &&
				ParsedTimeranges[i]->AbsoluteStartTime == InAbsoluteTimestamp &&
				ParsedTimeranges[i]->Duration == InDuration)
			{
				// Update the sequence index in case of looping. Otherwise the range will get removed when
				// the playback position is outside its range.
				ParsedTimeranges[i]->AbsoluteStartTime.SetSequenceIndex(InAbsoluteTimestamp.GetSequenceIndex());
				return;
			}
		}
		lock.Unlock();

		TSharedPtr<FParsedTimerange, ESPMode::ThreadSafe> ParsedTimerange = MakeShared<FParsedTimerange, ESPMode::ThreadSafe>();
		ParsedTimerange->Parser = ElectraWebVTTParser::IWebVTTParser::Create();
		ParsedTimerange->AbsoluteStartTime = InAbsoluteTimestamp;
		ParsedTimerange->Duration = InDuration;
		ParsedTimerange->SourceID = InAdditionalInfo.GetValue(ElectraSubtitleDecoderWVTTUtils::OptionKeys::SourceID).SafeGetFString();
		Electra::FParamDict noOpts;
		if (ParsedTimerange->Parser->ParseWebVTT(InData, noOpts))
		{
			FScopeLock lock2(&ParsedTimerangeLock);
			ParsedTimeranges.Emplace(MoveTemp(ParsedTimerange));
			ParsedTimeranges.Sort([](const TSharedPtr<FParsedTimerange, ESPMode::ThreadSafe>& a, const TSharedPtr<FParsedTimerange, ESPMode::ThreadSafe>& b){ return a->AbsoluteStartTime < b->AbsoluteStartTime;});

			// A change in parsed time ranges requires an immediate re-evaluation.
			NextEvaluationAt = FTimespan::MinValue();
		}
		else
		{
			UE_LOGF(LogElectraSubtitles, Error, "Bad WebVTT document, ignoring.");
			return;
		}
	}
	else
	{
		#define RETURN_IF_ERROR(expr)																\
			if (!expr)																				\
			{																						\
				UE_LOGF(LogElectraSubtitles, Error, "Bad WVTT text sample box, ignoring.");	\
				return;																				\
			}

		ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4 r(InData);

		struct FSubtitleEntry
		{
			FString Text;
			TOptional<int32> SourceID;
			TOptional<FString> CurrentTime;
			TOptional<FString> ID;
			TOptional<FString> Settings;
			bool bIsAdditionalCue = false;
		};

		// List of collected subtitles.
		TArray<FSubtitleEntry> Subtitles;
		// The currently worked-on subtitle.
		FSubtitleEntry Sub;

		bool bInsideCue = false;
		while(r.GetNumBytesRemaining() > 0)
		{
			uint32 BoxLen, BoxType;
			if (!r.Read(BoxLen) || !r.Read(BoxType))
			{
				UE_LOGF(LogElectraSubtitles, Error, "Bad WVTT text sample box, ignoring.");
				break;
			}
			if (BoxLen < 8 || (int32)(BoxLen-8 )> r.GetNumBytesRemaining())
			{
				UE_LOGF(LogElectraSubtitles, Error, "Bad WVTT text sample box, ignoring.");
				break;
			}

			// An empty cue?
			if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_vtte)
			{
				if (bInsideCue)
				{
					Subtitles.Emplace(MoveTemp(Sub));
				}
				bInsideCue = true;
				// This must be the only entry, so we stop parsing now.
				// If there are additional boxes this is an authoring error we ignore.
				break;
			}
			// An additional text box (a comment)?
			else if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_vtta)
			{
				if (bInsideCue)
				{
					Subtitles.Emplace(MoveTemp(Sub));
				}
				bInsideCue = true;
				Sub.bIsAdditionalCue = true;
				RETURN_IF_ERROR(r.ReadString(Sub.Text, BoxLen - 8));
			}
			// A cue?
			else if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_vttc)
			{
				if (bInsideCue)
				{
					Subtitles.Emplace(MoveTemp(Sub));
				}
				bInsideCue = true;
			}
			else if (bInsideCue)
			{
				// Cue source ID?
				if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_vsid)
				{
					int32 source_ID;
					RETURN_IF_ERROR(r.Read(source_ID));
					Sub.SourceID = source_ID;
				}
				// Cue time?
				else if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_ctim)
				{
					FString s;
					RETURN_IF_ERROR(r.ReadString(s, BoxLen - 8));
					Sub.CurrentTime = s;
				}
				// ID?
				else if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_iden)
				{
					FString s;
					RETURN_IF_ERROR(r.ReadString(s, BoxLen - 8));
					Sub.ID = s;
				}
				// Settings?
				else if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_sttg)
				{
					FString s;
					RETURN_IF_ERROR(r.ReadString(s, BoxLen - 8));
					Sub.Settings = s;
				}
				// Payload?
				else if (BoxType == ElectraSubtitleDecoderWVTTUtils::FDataReaderMP4::BoxType_payl)
				{
					RETURN_IF_ERROR(r.ReadString(Sub.Text, BoxLen - 8));
					// Strip any trailing newlines. They should not have been added by the muxing tool.
					while(!Sub.Text.IsEmpty() && (Sub.Text[Sub.Text.Len()-1] == TCHAR('\n') || Sub.Text[Sub.Text.Len()-1] == TCHAR('\r')))
					{
						Sub.Text.LeftChopInline(1);
					}
				}
				// Something else.
				else if (!r.ReadBytes(nullptr, BoxLen - 8))
				{
					break;
				}
			}
			// Something else.
			else if (!r.ReadBytes(nullptr, BoxLen - 8))
			{
				break;
			}
		}
		// At the end of the data add the currently worked on cue, if there is one.
		if (bInsideCue)
		{
			Subtitles.Emplace(MoveTemp(Sub));
		}

		for(auto &Cue : Subtitles)
		{
			if (!Cue.bIsAdditionalCue)
			{
				TSharedPtr<FSubtitleDecoderOutputWVTT, ESPMode::ThreadSafe> Out = MakeShared<FSubtitleDecoderOutputWVTT, ESPMode::ThreadSafe>();
				Out->SetTimestamp(InAbsoluteTimestamp);
				Out->SetDuration(InDuration);
				// This decoder returns plain text only. Remove all formatting tags.
				Out->SetText(ElectraWebVTTParser::IWebVTTParser::ProcessCueTextForType(Cue.Text, ElectraWebVTTParser::IWebVTTParser::EWebVTTType::Subtitle));
				if (Cue.ID.IsSet())
				{
					Out->SetID(Cue.ID.GetValue(), false);
				}
				else
				{
					Out->SetID(FString::Printf(TEXT("<%u>"), ++NextID), true);
				}
				ParsedSubtitleDelegate.Broadcast(Out);
			}
		}

		#undef RETURN_IF_ERROR
	}
}

void FElectraSubtitleDecoderWVTT::SignalStreamedSubtitleEOD()
{
}

void FElectraSubtitleDecoderWVTT::Flush()
{
	FScopeLock lock(&ParsedTimerangeLock);
	NextEvaluationAt = FTimespan::MinValue();
	LastSentSubtitle.Reset();
	LastPlaybackUpdateAbsPos.SetToInvalid();
	SendEmptySubtitleAt = FTimespan::MinValue();
}

void FElectraSubtitleDecoderWVTT::Start()
{
}

void FElectraSubtitleDecoderWVTT::Stop()
{
}

void FElectraSubtitleDecoderWVTT::UpdatePlaybackPosition(Electra::FTimeValue InAbsolutePosition, Electra::FTimeValue InLocalPosition)
{
	ParsedTimerangeLock.Lock();
	TArray<TSharedPtr<FParsedTimerange, ESPMode::ThreadSafe>> TimeRanges;
	for(int32 i=0; i<ParsedTimeranges.Num(); ++i)
	{
		if (InAbsolutePosition > ParsedTimeranges[i]->AbsoluteStartTime + ParsedTimeranges[i]->Duration &&
			InAbsolutePosition.GetSequenceIndex() >= ParsedTimeranges[i]->AbsoluteStartTime.GetSequenceIndex())
		{
			ParsedTimeranges.RemoveAt(i);
			--i;
		}
		else
		{
			TimeRanges.Emplace(ParsedTimeranges[i]);
		}
	}

	// Check if the sequence index has changed due to a seek or loop.
	// Typically we get a Flush() followed with another call here that is still prior to the loop point
	// which results in locking the current position to the end of the looping range since we are only
	// checking in the forward play direction.
	// When the loop actually happens we need to reset so we start at the beginning.
	if (!LastPlaybackUpdateAbsPos.IsValid() || LastPlaybackUpdateAbsPos.GetSequenceIndex() != InAbsolutePosition.GetSequenceIndex())
	{
		NextEvaluationAt = FTimespan::MinValue();
		SendEmptySubtitleAt = FTimespan::MinValue();
		for(int32 i=0; i<ParsedTimeranges.Num(); ++i)
		{
			ParsedTimeranges[i]->CurrentCueIterator.Reset();
		}
	}
	LastPlaybackUpdateAbsPos = InAbsolutePosition;

	FTimespan ThisEvalTime = NextEvaluationAt;
	FTimespan NextEval = NextEvaluationAt;
	bool bWasForcedEval = NextEvaluationAt == FTimespan::MinValue();
	ParsedTimerangeLock.Unlock();

	if (InAbsolutePosition.GetAsTimespan() < NextEval)
	{
		return;
	}

	// Get the current cues
	TArray<TUniquePtr<ElectraWebVTTParser::IWebVTTParser::ICue>> Cues;
	FTimespan ChangesAt;
	NextEval = FTimespan::MaxValue();
	for(int32 i=0; i<TimeRanges.Num(); ++i)
	{
		TimeRanges[i]->Parser->GetCuesAtTime(Cues, ChangesAt, TimeRanges[i]->CurrentCueIterator, InAbsolutePosition.GetAsTimespan());
		NextEval = FMath::Min(NextEval, ChangesAt);
	}
	ParsedTimerangeLock.Lock();
	NextEvaluationAt = NextEval;
	ParsedTimerangeLock.Unlock();
	if (Cues.Num())
	{
		// Start with this evaluation time as the new cue's start time.
		// If we have overlapping subtitles like
		// [a...........b]
		//       [c..d]
		// We need to split this into
		// [a..c], [c..d], [d..b]  and NOT
		// [a..c], [c..d], [a..b]  !!!
		FTimespan LargestCueStartTime = ThisEvalTime;
		FTimespan LargestCueEndTime = FTimespan::MinValue();
		FString CombinedPlainText;
		for(int32 i=0; i<Cues.Num(); ++i)
		{
			// Get the largest start time of the currently active cues.
			// This will be the start time of the new subtitle we send out.
			LargestCueStartTime = FMath::Max(LargestCueStartTime, Cues[i]->GetStartTime());
			LargestCueEndTime = FMath::Max(LargestCueStartTime, Cues[i]->GetEndTime());
			if (i > 0)
			{
				CombinedPlainText.Append(TEXT("\n"));
			}
			CombinedPlainText.Append(ElectraWebVTTParser::IWebVTTParser::ProcessCueTextForType(Cues[i]->GetText(), ElectraWebVTTParser::IWebVTTParser::EWebVTTType::Subtitle));
		}
		// Get the cue ID only when there is a single cue. If we had to aggregate cues then there is no unique ID.
		FString CueID = Cues.Num() == 1 ? Cues[0]->GetID() : FString();

		TSharedPtr<FSubtitleDecoderOutputWVTT, ESPMode::ThreadSafe> Out = MakeShared<FSubtitleDecoderOutputWVTT, ESPMode::ThreadSafe>();
		Out->SetText(CombinedPlainText);
		Electra::FTimeValue tv;
		tv.SetFromTimespan(LargestCueStartTime);
		tv.SetSequenceIndex(InAbsolutePosition.GetSequenceIndex());
		Out->SetTimestamp(tv);
		Electra::FTimeValue dur;
		// The duration of the subtitle we send is from the start time to the
		// next evaluation time at which point we will construct a new aggegrated subtitle.
		if (NextEval < FTimespan::MaxValue())
		{
			dur.SetFromTimespan(NextEval - LargestCueStartTime);
		}
		else
		{
			dur.SetFromSeconds(0.5);
		}
		Out->SetDuration(dur);
		Out->SetID(CueID.Len() ? CueID : FString::Printf(TEXT("<%u>"), ++NextID), CueID.IsEmpty());

		// Do not send the same subtitle again. This happens when adding a new streamed subtitle had to invalidate the
		// time at which we had to update.
		if (bWasForcedEval && LastSentSubtitle.IsValid() && *LastSentSubtitle == *Out)
		{
			Out.Reset();
		}
		if (Out.IsValid())
		{
			ParsedSubtitleDelegate.Broadcast(Out);
			LastSentSubtitle = Out;
		}

		if (bSendEmptySubtitleDuringGaps)
		{
			SendEmptySubtitleAt = LargestCueEndTime;
		}
	}
	else if (bSendEmptySubtitleDuringGaps && SendEmptySubtitleAt != FTimespan::MinValue())
	{
		TSharedPtr<FSubtitleDecoderOutputWVTT, ESPMode::ThreadSafe> Out = MakeShared<FSubtitleDecoderOutputWVTT, ESPMode::ThreadSafe>();
		Electra::FTimeValue tv;
		tv.SetFromTimespan(SendEmptySubtitleAt);
		tv.SetSequenceIndex(InAbsolutePosition.GetSequenceIndex());
		Out->SetTimestamp(tv);
		FTimespan d(NextEval - SendEmptySubtitleAt);
		if (d < FTimespan::Zero() || d.GetTicks() > ETimespan::TicksPerSecond)
		{
			d = FTimespan(ETimespan::TicksPerSecond);
		}
		Electra::FTimeValue dur;
		dur.SetFromTimespan(d);
		Out->SetDuration(dur);
		Out->SetID(FString::Printf(TEXT("<%u>"), ++NextID), true);
		ParsedSubtitleDelegate.Broadcast(Out);

		SendEmptySubtitleAt = FTimespan::MinValue();
	}
}
