// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraSubtitleDecoder.h"
#include "wvtt/WebVTTParser.h"

class IElectraSubtitleDecoderFactoryRegistry;
class FSubtitleDecoderOutputWVTT;



/**
 * WebVTT subtitle decoder (https://www.w3.org/TR/webvtt1/)
 */
class FElectraSubtitleDecoderWVTT : public IElectraSubtitleDecoder
{
public:
	static void RegisterCodecs(IElectraSubtitleDecoderFactoryRegistry& InRegistry);

	FElectraSubtitleDecoderWVTT();

	virtual ~FElectraSubtitleDecoderWVTT();

	//-------------------------------------------------------------------------
	// Methods from IElectraSubtitleDecoder
	//
	virtual bool InitializeStreamWithCSD(const TArray<uint8>& InCSD, const Electra::FParamDict& InAdditionalInfo) override;
	virtual IElectraSubtitleDecoder::FOnSubtitleReceivedDelegate& GetParsedSubtitleReceiveDelegate() override;
	virtual Electra::FTimeValue GetStreamedDeliveryTimeOffset() override;
	virtual void AddStreamedSubtitleData(const TArray<uint8>& InData, Electra::FTimeValue InAbsoluteTimestamp, Electra::FTimeValue InDuration, const Electra::FParamDict& InAdditionalInfo) override;
	virtual void SignalStreamedSubtitleEOD() override;
	virtual void Flush() override;
	virtual void Start() override;
	virtual void Stop() override;
	virtual void UpdatePlaybackPosition(Electra::FTimeValue InAbsolutePosition, Electra::FTimeValue InLocalPosition) override;

private:

	struct FParsedTimerange
	{
		TSharedPtr<ElectraWebVTTParser::IWebVTTParser, ESPMode::ThreadSafe> Parser;
		TUniquePtr<ElectraWebVTTParser::IWebVTTParser::ICueIterator> CurrentCueIterator;
		Electra::FTimeValue AbsoluteStartTime;
		Electra::FTimeValue Duration;
		FString SourceID;
	};
	FCriticalSection ParsedTimerangeLock;
	TArray<TSharedPtr<FParsedTimerange, ESPMode::ThreadSafe>> ParsedTimeranges;
	FTimespan NextEvaluationAt { FTimespan::MinValue() };
	TSharedPtr<FSubtitleDecoderOutputWVTT, ESPMode::ThreadSafe> LastSentSubtitle;
	Electra::FTimeValue LastPlaybackUpdateAbsPos;

	FOnSubtitleReceivedDelegate ParsedSubtitleDelegate;
	uint32 NextID = 0;
	bool bNeedsParsing = false;
	bool bSendEmptySubtitleDuringGaps = false;
	FTimespan SendEmptySubtitleAt { FTimespan::MinValue() };
};
