// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebVTTParser.h"
#include "ElectraSubtitleUtils.h"
#include "ParameterDictionary.h"

namespace ElectraWebVTTParser
{
const TCHAR* const ConstWEBVTT(TEXT("WEBVTT"));
const TCHAR* const ConstSTYLE(TEXT("STYLE"));
const TCHAR* const ConstREGION(TEXT("REGION"));
const TCHAR* const ConstNOTE(TEXT("NOTE"));
const TCHAR* const ConstArrow(TEXT("-->"));

const TCHAR* const ConstHLSTimestampMap(TEXT("X-TIMESTAMP-MAP"));
const TCHAR* const ConstHLSTimestampLOCAL(TEXT("LOCAL"));
const TCHAR* const ConstHLSTimestampMPEGTS(TEXT("MPEGTS"));

const TCHAR* const ConstHLSHandleTimestampMapping(TEXT("handle_ts_mapping"));

/*********************************************************************************************************************/

FString IWebVTTParser::ProcessCueTextForType(FStringView InText, EWebVTTType InType)
{
	if (InText.Len() == 0)
	{
		return FString();
	}
	if (InType != EWebVTTType::Subtitle)
	{
		return FString(MoveTemp(InText));
	}
	// See if there is any '<' character that indicates a potential span
	// or an '&' indicating a HTML character reference (like "&amp;" or "&gt;")
	int32 LtPos = INDEX_NONE;
	int32 AmpPos = INDEX_NONE;
	if (!InText.FindChar(TCHAR('<'), LtPos) && !InText.FindChar(TCHAR('&'), AmpPos))
	{
		// With neither present we return an empty string to indicate the clean text
		// is identical to the cue text. No need to duplicate anyting.
		return FString(MoveTemp(InText));
	}
	TArray<TCHAR> New;
	New.Reserve(InText.Len());
	const TCHAR* Text = &InText[0];
	const TCHAR* TextEnd = Text + InText.Len();
	while(Text < TextEnd)
	{
		if (*Text == TCHAR('<'))
		{
			// Skip over whatever span this is.
			while(Text < TextEnd && *Text != TCHAR('>'))
			{
				++Text;
			}
			++Text;
		}
		else if (*Text == TCHAR('&'))
		{
			const TCHAR* HtmlCharRef = Text + 1;
			// Skip until we find the terminating ';'
			while(Text < TextEnd && *Text != TCHAR(';'))
			{
				++Text;
			}
			if (Text < TextEnd)
			{
				FStringView CharRef = MakeStringView(HtmlCharRef, Text - HtmlCharRef);
				++Text;

				// We handle only very few named character references that are most commonly used.
				// The list of all of them (https://html.spec.whatwg.org/multipage/named-characters.html#named-character-references)
				// is too exhaustive to handle.
				// We also completely ignore numeric character references.
				if (CharRef.Equals(TEXT("lt"), ESearchCase::IgnoreCase))
				{
					New.Add(TCHAR('<'));
				}
				else if (CharRef.Equals(TEXT("gt"), ESearchCase::IgnoreCase))
				{
					New.Add(TCHAR('>'));
				}
				else if (CharRef.Equals(TEXT("amp"), ESearchCase::IgnoreCase))
				{
					New.Add(TCHAR('&'));
				}
				else if (CharRef.Equals(TEXT("quot"), ESearchCase::IgnoreCase))
				{
					New.Add(TCHAR('"'));
				}
				else if (CharRef.Equals(TEXT("nbsp"), ESearchCase::IgnoreCase))
				{
					New.Add(TCHAR(' '));
				}
			}
		}
		else
		{
			New.Add(*Text++);
		}
	}
	return FString(MakeStringView(New.GetData(), New.Num()));
}

/*********************************************************************************************************************/

class FWebVTTParser : public IWebVTTParser
{
public:
	FWebVTTParser();
	virtual ~FWebVTTParser();

	const FString& GetLastErrorMessage() const override
	{ return LastErrorMsg; }
	bool ParseWebVTT(const TArray<uint8>& InWebVTTData, const Electra::FParamDict& InOptions) override;
	void GetCuesAtTime(TArray<TUniquePtr<ICue>>& OutCues, FTimespan& OutNextChangeAt, TUniquePtr<ICueIterator>& InOutIterator, const FTimespan& InAtTime) override;

	struct FWebVTTDocument
	{
		struct FMetadataHeader
		{
			FString Header;
			FString Value;
		};

		struct FRegion
		{
			struct FAnchor
			{
				double x = 0.0;
				double y = 100.0;
			};
			FString id;
			double width = 100.0;
			int32 lines = 3;
			FAnchor regionAnchor;
			FAnchor viewportAnchor;
			bool bScrollUp = false;
		};

		struct FCue
		{
			FString ID;
			FString Text;
			FTimespan Start;
			FTimespan End;
			struct FSettings
			{
				enum class ELayout
				{
					Horizontal,
					VerticalGrowingLeft,
					VerticalGrowingRight
				};
				struct FLine
				{
					enum class EAlignment
					{
						Start,
						Center,
						End
					};
					double offset = 0.0;
					EAlignment alignment = EAlignment::Start;
					bool bIsPercentage = false;
				};
				struct FPosition
				{
					enum class EAlignment
					{
						Auto,
						LineLeft,
						Center,
						LineRight
					};
					double position = 0.0;
					EAlignment alignment = EAlignment::Auto;
				};
				enum class EAlignment
				{
					Start,
					Center,
					End,
					Left,
					Right
				};
				FLine line;
				FPosition position;
				FString region;
				ELayout layout = ELayout::Horizontal;
				EAlignment alignment = EAlignment::Center;
				double size = 100.0;
			};
			FSettings Settings;
		};

		TArray<FMetadataHeader> MetadataHeaders;
		TArray<FString> CSSStyles;
		TArray<FRegion> Regions;
		TArray<TSharedPtr<FCue, ESPMode::ThreadSafe>> Cues;
		FTimespan SmallestCueStartTime { FTimespan::MaxValue() };
		FTimespan LargestCueEndTime { FTimespan::MinValue() };

		FTimespan TimestampOffset;

	};


	class FPublicCue : public ICue
	{
	public:
		FPublicCue(const TSharedPtr<FWebVTTDocument::FCue, ESPMode::ThreadSafe>& InFromCue, const FTimespan& InTimeOffset)
			: Cue(InFromCue), TimeOffset(InTimeOffset)
		{ }
		~FPublicCue() = default;
		FTimespan GetStartTime() const override
		{
			return Cue->Start + TimeOffset;
		}
		FTimespan GetEndTime() const override
		{
			return Cue->End + TimeOffset;
		}
		FString GetID() const override
		{
			return Cue->ID;
		}
		FString GetText() const override
		{
			return Cue->Text;
		}
		bool operator == (const FPublicCue& Other) const
		{
			return Cue->Start == Other.Cue->Start && Cue->End == Other.Cue->End && Cue->ID.Equals(Other.Cue->ID) && Cue->Text.Equals(Other.Cue->Text);
		}
	private:
		TSharedPtr<FWebVTTDocument::FCue, ESPMode::ThreadSafe> Cue;
		FTimespan TimeOffset;
	};

	class FPublicCueIterator : public ICueIterator
	{
	public:
		virtual ~FPublicCueIterator()
		{ }

		void Reset()
		{
			CurrentTime = FTimespan::MinValue();
			CurrentIndex = 0;
		}

		FTimespan CurrentTime { FTimespan::MinValue() };
		int32 CurrentIndex { 0 };
	};


private:

	static bool IsAllNumeric(FStringView In)
	{
		for(auto& Ch : In)
		{
			if (!FChar::IsDigit(Ch))
			{
				return false;
			}
		}
		return true;
	}

	static bool IsAllNumericMaybeNeg(FStringView In)
	{
		if (In[0] == TCHAR('-'))
		{
			for(int32 i=1; i<In.Len(); ++i)
			{
				if (!FChar::IsDigit(In[i]))
				{
					return false;
				}
			}
			return true;
		}
		return IsAllNumeric(In);
	}

	static bool IsAllNumericDot(FStringView In)
	{
		int32 NumDots = 0;
		for(auto& Ch : In)
		{
			NumDots += Ch == TCHAR('.') ? 1 : 0;
			if (!FChar::IsDigit(Ch) && Ch != TCHAR('.'))
			{
				return false;
			}
		}
		return NumDots <= 1;
	}

	static bool IsAllNumericMaybeNegDot(FStringView In)
	{
		int32 NumDots = 0;
		for(int32 i=0; i<In.Len(); ++i)
		{
			if (i == 0 && In[0] == TCHAR('-'))
			{
				continue;
			}
			NumDots += In[i] == TCHAR('.') ? 1 : 0;
			if (!FChar::IsDigit(In[i]) && In[i] != TCHAR('.'))
			{
				return false;
			}
		}
		return NumDots <= 1;
	}

	static bool IsNewline(const FString::TConstIterator& InIt)
	{
		return *InIt == TCHAR('\n') || *InIt == TCHAR('\r');
	}

	static bool LocateEndOfLine(FString::TConstIterator& InOutIt)
	{
		bool bHasNonWS = false;
		while(InOutIt && !IsNewline(InOutIt))
		{
			bHasNonWS = bHasNonWS || !FChar::IsWhitespace(*InOutIt);
			++InOutIt;
		}
		return !bHasNonWS;
	}

	static int32 SkipOverLinebreaks(FString::TConstIterator& InOutIt)
	{
		int32 NumEOLs = 0;
		while(InOutIt)
		{
			if (*InOutIt == TCHAR('\r'))
			{
				++NumEOLs;
				++InOutIt;
				if (InOutIt && *InOutIt == TCHAR('\n'))
				{
					++InOutIt;
				}
			}
			else if (*InOutIt == TCHAR('\n'))
			{
				++NumEOLs;
				++InOutIt;
			}
			else
			{
				break;
			}
		}
		return NumEOLs;
	}

	static FStringView CreateView(const FString::TConstIterator& sIt, const FString::TConstIterator& eIt)
	{
		return MakeStringView(sIt.operator->(), eIt.GetIndex() - sIt.GetIndex());
	}

	static int32 SplitAtAndRemoveWhitespace(TArray<FStringView>& OutParts, bool& bInOutContainsArrow, FStringView In)
	{
		OutParts.Empty();
		bInOutContainsArrow = false;
		int32 sIdx = -1;
		for(int32 i=0; i<In.Len(); ++i)
		{
			if (!FChar::IsWhitespace(In[i]))
			{
				if (sIdx < 0)
				{
					sIdx = i;
				}
			}
			else if (sIdx >= 0)
			{
				int32 Len = i - sIdx;
				OutParts.Emplace(MakeStringView(&In[sIdx], Len));
				bInOutContainsArrow = bInOutContainsArrow || OutParts.Last().Equals(ConstArrow);
				sIdx = -1;
			}
		}
		if (sIdx >= 0)
		{
			OutParts.Emplace(MakeStringView(&In[sIdx], In.Len() - sIdx));
			bInOutContainsArrow = bInOutContainsArrow || OutParts.Last().Equals(ConstArrow);
		}
		return OutParts.Num();
	}

	static bool ParseLineIntoPartsAtWhitespace(TArray<FStringView>& OutParts, FString::TConstIterator& InOutIt)
	{
		FString::TConstIterator Start(InOutIt);
		LocateEndOfLine(InOutIt);
		bool bContainsArrow;
		SplitAtAndRemoveWhitespace(OutParts, bContainsArrow, CreateView(Start, InOutIt));
		return bContainsArrow;
	}

	static bool ParseTimestamp(FTimespan& OutTime, FStringView In)
	{
		FString Time(In);
		TArray<FString> Parts;
		Time.ParseIntoArray(Parts, TEXT(":"), false);
		if (Parts.Num() < 2 || Parts.Num() > 3)
		{
			return false;
		}
		// Minutes, Seconds and fractional seconds are mandatory.
		// If there are no hours we add a 0 for hours to simplify parsing.
		if (Parts.Num() == 2)
		{
			Parts.Insert(TEXT("0"), 0);
		}
		if (!IsAllNumeric(Parts[0]) || !IsAllNumeric(Parts[1]) || !IsAllNumericDot(Parts[2]))
		{
			return false;
		}
		int32 DotPos;
		if (!Parts[2].FindChar(TCHAR('.'), DotPos) || DotPos != 2)
		{
			return false;
		}
		Parts.Add(Parts[2].Mid(DotPos + 1));
		Parts[2].LeftInline(DotPos);
		if (Parts[1].Len() != 2 || Parts[2].Len() != 2 || Parts[3].Len() != 3)
		{
			return false;
		}
		int32 h = FCString::Atoi(*Parts[0]);
		int32 m = FCString::Atoi(*Parts[1]);
		int32 s = FCString::Atoi(*Parts[2]);
		int32 ms = FCString::Atoi(*Parts[3]);
		if (m > 59 || s > 59)
		{
			return false;
		}
		OutTime = FTimespan(0, h, m, s, ms * 1000000);
		return true;
	}


	TUniquePtr<FWebVTTDocument> CurrentDoc;

	FString LastErrorMsg;
	bool bInCues = false;
};


/*********************************************************************************************************************/

TSharedPtr<IWebVTTParser, ESPMode::ThreadSafe> IWebVTTParser::Create()
{
	return MakeShared<FWebVTTParser, ESPMode::ThreadSafe>();
}

FWebVTTParser::FWebVTTParser()
{
}

FWebVTTParser::~FWebVTTParser()
{
}




bool FWebVTTParser::ParseWebVTT(const TArray<uint8>& InWebVTTData, const Electra::FParamDict& InOptions)
{
	if (InWebVTTData.Num() < 6)
	{
		return false;
	}
	bool bHaveBOM = InWebVTTData[0] == 0xef && InWebVTTData[1] == 0xbb && InWebVTTData[2] == 0xbf;
	// The data is expected to be a UTF-8 encoded string. If there is a BOM we skip over it.
	FString Document = FString::ConstructFromPtrSize((const UTF8CHAR*)InWebVTTData.GetData() + (bHaveBOM ? 3 : 0), InWebVTTData.Num() - (bHaveBOM ? 3 : 0));
	if (Document.Len() < 6)
	{
		return false;
	}

	// Set up iterator to the start of the document and start checking it to contain a proper signature
	TArray<FStringView> Parts;
	FString::TConstIterator It(Document.CreateConstIterator());
	ParseLineIntoPartsAtWhitespace(Parts, It);
	if (Parts.IsEmpty() || Parts.Num() > 1 || !Parts[0].Equals(ConstWEBVTT, ESearchCase::CaseSensitive))
	{
		return false;
	}

	TUniquePtr<FWebVTTDocument> NewDoc = MakeUnique<FWebVTTDocument>();

	// We do not check the document for validity, just parse it line for line.
	FString PreviousUnknownLine;
	bool bSkipUntilEmptyLine = false;
	while(It)
	{
		if (bSkipUntilEmptyLine)
		{
			// The assumption is that the iterator is currently pointing to the linebreak
			// at the end of the line asking to skip ahead.
			// So we first skip over linebreaks and if that's more than just one we are done.
			// If not we check if the line is empty (all whitespaces) which is also good.
			if (SkipOverLinebreaks(It) > 1 || LocateEndOfLine(It))
			{
				bSkipUntilEmptyLine = false;
			}
			continue;
		}

		// Move on to the next line by skipping over CR/LF sequences. This does not skip
		// over lines that contain whitespace.
		int32 NumLineBreaks = SkipOverLinebreaks(It);
		if (It)
		{
			// If we skipped more than one line break then the previous line is of no interest any more.
			if (NumLineBreaks > 1)
			{
				PreviousUnknownLine.Empty();
			}
			FString::TConstIterator LineStart(It);
			bool bContainsArrow = ParseLineIntoPartsAtWhitespace(Parts, It);
			if (bContainsArrow)
			{
				if (Parts.Num() < 3 || !Parts[1].Equals(ConstArrow))
				{
					FString cue(CreateView(LineStart, It));
					LastErrorMsg = FString::Printf(TEXT("\"%s\" is not a cue"), *cue);
					return false;
				}
				else
				{
					bInCues = true;

					TSharedPtr<FWebVTTDocument::FCue, ESPMode::ThreadSafe> NewCue = MakeShared<FWebVTTDocument::FCue, ESPMode::ThreadSafe>();
					NewCue->ID = MoveTemp(PreviousUnknownLine);
					if (!ParseTimestamp(NewCue->Start, Parts[0]))
					{
						LastErrorMsg = FString::Printf(TEXT("\"%s\" is not a valid timestamp"), *FString(Parts[0]));
						return false;
					}
					if (!ParseTimestamp(NewCue->End, Parts[2]))
					{
						LastErrorMsg = FString::Printf(TEXT("\"%s\" is not a valid timestamp"), *FString(Parts[1]));
						return false;
					}
					// Parse cue settings
					for(int32 np=3; np<Parts.Num(); ++np)
					{
						int32 ColonPos = INDEX_NONE;
						if (Parts[np].FindChar(TCHAR(':'), ColonPos) && ColonPos > 0 && ColonPos+1 < Parts[np].Len())
						{
							auto Key = Parts[np].Left(ColonPos);
							auto Value = Parts[np].Mid(ColonPos + 1);
							if (Key.Equals(TEXT("region"), ESearchCase::CaseSensitive))
							{
								NewCue->Settings.region = Value;
							}
							else if (Key.Equals(TEXT("vertical"), ESearchCase::CaseSensitive))
							{
								if (Value.Equals(TEXT("lr"), ESearchCase::CaseSensitive))
								{
									NewCue->Settings.layout = FWebVTTDocument::FCue::FSettings::ELayout::VerticalGrowingRight;
								}
								else if (Value.Equals(TEXT("rl"), ESearchCase::CaseSensitive))
								{
									NewCue->Settings.layout = FWebVTTDocument::FCue::FSettings::ELayout::VerticalGrowingLeft;
								}
							}
							else if (Key.Equals(TEXT("line"), ESearchCase::CaseSensitive))
							{
								FString SubPart(Value);
								TArray<FString> SubParts;
								SubPart.ParseIntoArray(SubParts, TEXT(","), true);
								bool bCont = false;
								if (SubParts[0].EndsWith(TEXT("%")))
								{
									SubParts[0].LeftChopInline(1);
									if (IsAllNumericDot(SubParts[0]))
									{
										double Pct = FCString::Atod(*SubParts[0]);
										if (Pct >= 0.0 && Pct <= 100.0)
										{
											NewCue->Settings.line.offset = Pct;
											NewCue->Settings.line.bIsPercentage = true;
											bCont = true;
										}
									}
								}
								else if (IsAllNumericMaybeNegDot(SubParts[0]))
								{
									NewCue->Settings.line.offset = FCString::Atod(*SubParts[0]);
									NewCue->Settings.line.bIsPercentage = false;
									bCont = true;
								}
								if (bCont && SubParts.Num() > 1)
								{
									if (SubParts[1].Equals(TEXT("start"), ESearchCase::CaseSensitive))
									{
										NewCue->Settings.line.alignment = FWebVTTDocument::FCue::FSettings::FLine::EAlignment::Start;
									}
									else if (SubParts[1].Equals(TEXT("center"), ESearchCase::CaseSensitive))
									{
										NewCue->Settings.line.alignment = FWebVTTDocument::FCue::FSettings::FLine::EAlignment::Center;
									}
									else if (SubParts[1].Equals(TEXT("end"), ESearchCase::CaseSensitive))
									{
										NewCue->Settings.line.alignment = FWebVTTDocument::FCue::FSettings::FLine::EAlignment::End;
									}
								}
							}
							else if (Key.Equals(TEXT("position"), ESearchCase::CaseSensitive))
							{
								FString SubPart(Value);
								TArray<FString> SubParts;
								SubPart.ParseIntoArray(SubParts, TEXT(","), true);
								if (SubParts[0].EndsWith(TEXT("%")))
								{
									SubParts[0].LeftChopInline(1);
									if (IsAllNumericDot(SubParts[0]))
									{
										double Pct = FCString::Atod(*SubParts[0]);
										if (Pct >= 0.0 && Pct <= 100.0)
										{
											NewCue->Settings.position.position = Pct;
											if (SubParts.Num() > 1)
											{
												if (SubParts[1].Equals(TEXT("line-left"), ESearchCase::CaseSensitive))
												{
													NewCue->Settings.position.alignment = FWebVTTDocument::FCue::FSettings::FPosition::EAlignment::LineLeft;
												}
												else if (SubParts[1].Equals(TEXT("center"), ESearchCase::CaseSensitive))
												{
													NewCue->Settings.position.alignment = FWebVTTDocument::FCue::FSettings::FPosition::EAlignment::Center;
												}
												else if (SubParts[1].Equals(TEXT("line-right"), ESearchCase::CaseSensitive))
												{
													NewCue->Settings.position.alignment = FWebVTTDocument::FCue::FSettings::FPosition::EAlignment::LineRight;
												}
											}
										}
									}
								}
							}
							else if (Key.Equals(TEXT("align"), ESearchCase::CaseSensitive))
							{
								if (Value.Equals(TEXT("start"), ESearchCase::CaseSensitive))
								{
									NewCue->Settings.alignment = FWebVTTDocument::FCue::FSettings::EAlignment::Start;
								}
								else if (Value.Equals(TEXT("center"), ESearchCase::CaseSensitive))
								{
									NewCue->Settings.alignment = FWebVTTDocument::FCue::FSettings::EAlignment::Center;
								}
								else if (Value.Equals(TEXT("end"), ESearchCase::CaseSensitive))
								{
									NewCue->Settings.alignment = FWebVTTDocument::FCue::FSettings::EAlignment::Right;
								}
								else if (Value.Equals(TEXT("left"), ESearchCase::CaseSensitive))
								{
									NewCue->Settings.alignment = FWebVTTDocument::FCue::FSettings::EAlignment::Left;
								}
								else if (Value.Equals(TEXT("right"), ESearchCase::CaseSensitive))
								{
									NewCue->Settings.alignment = FWebVTTDocument::FCue::FSettings::EAlignment::Right;
								}
							}
							else if (Key.Equals(TEXT("size"), ESearchCase::CaseSensitive))
							{
								if (Value.EndsWith(TEXT("%")))
								{
									FString Num(Value);
									Num.LeftChopInline(1);
									if (IsAllNumericDot(Num))
									{
										double Pct = FCString::Atod(*Num);
										if (Pct >= 0.0 && Pct <= 100.0)
										{
											NewCue->Settings.size = Pct;
										}
									}
								}
							}
						}
					}

					// Parse the actual cue text now
					while(1)
					{
						// Skip over the line break at the end of the line.
						if (SkipOverLinebreaks(It) > 1)
						{
							// If there was more than one line break the cue is empty.
							break;
						}
						FString::TConstIterator CueLineStart(It);
						// Go to the end of the line. If it is empty we are done.
						if (LocateEndOfLine(It))
						{
							break;
						}
						// If there is text in the cue we need to append a line break
						if (NewCue->Text.Len())
						{
							NewCue->Text.Append(TEXT("\n"));
						}
						FStringView Text(CreateView(CueLineStart, It));
						NewCue->Text.Append(Text);
					}

					// Add the new cue to the list.
					if (NewCue->Start < NewDoc->SmallestCueStartTime)
					{
						NewDoc->SmallestCueStartTime = NewCue->Start;
					}
					if (NewCue->End > NewDoc->LargestCueEndTime)
					{
						NewDoc->LargestCueEndTime = NewCue->End;
					}
					NewDoc->Cues.Emplace(MoveTemp(NewCue));
				}
			}
			else if (Parts.Num())
			{
				PreviousUnknownLine.Empty();
				// Is this a comment?
				if (Parts[0].Equals(ConstNOTE, ESearchCase::CaseSensitive))
				{
					// Comments just get skipped.
					bSkipUntilEmptyLine = true;
				}
				// A style block?
				else if (Parts[0].Equals(ConstSTYLE, ESearchCase::CaseSensitive))
				{
					// Skip over the line break at the end of the line.
					if (SkipOverLinebreaks(It) > 1)
					{
						// If there was more than one line break the style block is empty.
					}
					else
					{
						// We need to collect anything until an empty line, but not parse it.
						// This is pure CSS which is not meant for us to interpret.
						FString::TConstIterator CSSStart(It);
						while(1)
						{
							bool bEmptyLine = LocateEndOfLine(It);
							if (bEmptyLine || SkipOverLinebreaks(It) > 1)
							{
								break;
							}
						}
						if (It)
						{
							NewDoc->CSSStyles.Emplace(CreateView(CSSStart, It));
						}
					}
				}
				// A region block?
				else if (Parts[0].Equals(ConstREGION, ESearchCase::CaseSensitive))
				{
					// Skip over the line break at the end of the line.
					if (SkipOverLinebreaks(It) > 1)
					{
						// If there was more than one line break the region block is empty.
					}
					else
					{
						FWebVTTDocument::FRegion NewRegion;
						while(1)
						{
							ParseLineIntoPartsAtWhitespace(Parts, It);
							if (Parts.IsEmpty())
							{
								break;
							}
							for(int32 i=0; i<Parts.Num(); ++i)
							{
								int32 ColonPos = INDEX_NONE;
								if (Parts[i].FindChar(TCHAR(':'), ColonPos) && ColonPos > 0 && ColonPos+1 < Parts[i].Len())
								{
									auto Key = Parts[i].Left(ColonPos);
									auto Value = Parts[i].Mid(ColonPos + 1);
									if (Key.Equals(TEXT("id"), ESearchCase::CaseSensitive))
									{
										NewRegion.id = Value;
									}
									else if (Key.Equals(TEXT("width"), ESearchCase::CaseSensitive))
									{
										if (Value[Value.Len() - 1] == TCHAR('%'))
										{
											auto Num = Value.Left(Value.Len() - 1);
											if (IsAllNumericDot(Num))
											{
												double Pct = FCString::Atod(&Num[0]);
												if (Pct >= 0.0 && Pct <= 100.0)
												{
													NewRegion.width = Pct;
												}
											}
										}
									}
									else if (Key.Equals(TEXT("lines"), ESearchCase::CaseSensitive))
									{
										if (IsAllNumeric(Value))
										{
											NewRegion.lines = FCString::Atoi(&Value[0]);
										}
									}
									else if (Key.Equals(TEXT("regionanchor"), ESearchCase::CaseSensitive) ||
											 Key.Equals(TEXT("viewportanchor"), ESearchCase::CaseSensitive))
									{
										FString SubPart(Value);
										TArray<FString> SubParts;
										SubPart.ParseIntoArray(SubParts, TEXT(","), true);
										if (SubParts.Num() == 2 && SubParts[0].EndsWith(TEXT("%")) && SubParts[1].EndsWith(TEXT("%")))
										{
											SubParts[0].LeftChopInline(1);
											SubParts[1].LeftChopInline(1);
											if (IsAllNumericDot(SubParts[0]) && IsAllNumericDot(SubParts[1]))
											{
												double PctX = FCString::Atod(*SubParts[0]);
												double PctY = FCString::Atod(*SubParts[1]);
												if (PctX >= 0.0 && PctX <= 100.0 && PctY >= 0.0 && PctY <= 100.0)
												{
													if (Key[0] == TCHAR('r'))
													{
														NewRegion.regionAnchor.x = PctX;
														NewRegion.regionAnchor.y = PctY;
													}
													else
													{
														NewRegion.viewportAnchor.x = PctX;
														NewRegion.viewportAnchor.y = PctY;
													}
												}
											}
										}
									}
									else if (Key.Equals(TEXT("scroll"), ESearchCase::CaseSensitive))
									{
										NewRegion.bScrollUp = Value.Equals(TEXT("up"), ESearchCase::CaseSensitive);
									}
								}
							}
							if (SkipOverLinebreaks(It) > 1)
							{
								break;
							}
						}
						NewDoc->Regions.Emplace(MoveTemp(NewRegion));
					}
				}
				else
				{
					// Something else.
					// Check if this is a (deprecated since 2016 but still in use with old files) metadata header.
					// Metadata headers cannot appear once a cue has been found.
					auto Line(CreateView(LineStart, It));

					/*
						Note: The old WebVTT spec that still contained metadata headers says that the header
						      and value are separated by a COLON
						        https://www.w3.org/TR/2015/WD-webvtt1-20151208/#webvtt-metadata-header
						      which I'm sure is right. The only metadata we are interested in at this point
							  is Apple's HLS "X-TIMESTAMP-MAP" and this uses an EQUALS sign.
							  Yes, there are colons in there as well, but apparently not to separate header
							  and value.
							  We have seen differently structured headers as well like:
									X-TIMESTAMP-MAP=LOCAL:00:00:00.000,MPEGTS:9000
								and
									X-TIMESTAMP-MAP=MPEGTS:900000,LOCAL:00:00:00.000

							  so clearly the separator here is an equals sign.
					*/

					int32 ColonPos, EqualsPos;
					Line.FindChar(TCHAR('='), EqualsPos);
					Line.FindChar(TCHAR(':'), ColonPos);
					// We look at both colon and equal and use whichever one appears first.
					if (!bInCues && (ColonPos != INDEX_NONE || EqualsPos != INDEX_NONE))
					{
						int32 HeaderSepPos = (ColonPos != INDEX_NONE && EqualsPos != INDEX_NONE) ? (EqualsPos < ColonPos ? EqualsPos : ColonPos) : (ColonPos != INDEX_NONE ? ColonPos : EqualsPos);
						// The old WebVTT spec says nothing about skipping leading or trailing whitespace
						// so we leave them in!
						FWebVTTDocument::FMetadataHeader& Hdr(NewDoc->MetadataHeaders.Emplace_GetRef());
						Hdr.Header = Line.Left(HeaderSepPos);
						Hdr.Value = Line.Mid(HeaderSepPos + 1);
					}
					else
					{
						PreviousUnknownLine = Line;
						PreviousUnknownLine.TrimStartAndEndInline();
					}
				}
			}
		}
	}

	// Check if there is a timestamp mapping as it may occur with HLS.
	if (InOptions.GetValue(ElectraWebVTTParser::ConstHLSHandleTimestampMapping).SafeGetBool())
	{
		/*
			Look for the mapping as described in the HLS RFC.
			There are two versions of this around:
				X-TIMESTAMP-MAP=LOCAL:00:00:00.000,MPEGTS:9000
			and
				X-TIMESTAMP-MAP=MPEGTS:900000,LOCAL:00:00:00.000
		*/
		const FWebVTTDocument::FMetadataHeader* HLSTimestampMapping = NewDoc->MetadataHeaders.FindByPredicate([](const FWebVTTDocument::FMetadataHeader& h){ return h.Header.Equals(ConstHLSTimestampMap); });
		if (HLSTimestampMapping)
		{
			TArray<FString> MappingParts;
			HLSTimestampMapping->Value.ParseIntoArray(MappingParts, TEXT(","), true);
			if (MappingParts.Num() == 2)
			{
				int32 ColonPos1, ColonPos2;
				MappingParts[0].FindChar(TCHAR(':'), ColonPos1);
				MappingParts[1].FindChar(TCHAR(':'), ColonPos2);
				if (ColonPos1 != INDEX_NONE && ColonPos2 != INDEX_NONE)
				{
					TArray<FString> Values[2];
					Values[0].Emplace(MappingParts[0].Left(ColonPos1));
					Values[0].Emplace(MappingParts[0].Mid(ColonPos1 + 1));
					Values[1].Emplace(MappingParts[1].Left(ColonPos2));
					Values[1].Emplace(MappingParts[1].Mid(ColonPos2 + 1));
					int32 LocalIdx = Values[0][0].Equals(ConstHLSTimestampLOCAL) ? 0 : Values[1][0].Equals(ConstHLSTimestampLOCAL) ? 1 : -1;
					int32 MpegIdx = Values[0][0].Equals(ConstHLSTimestampMPEGTS) ? 0 : Values[1][0].Equals(ConstHLSTimestampMPEGTS) ? 1 : -1;
					if (LocalIdx != -1 && MpegIdx != -1)
					{
						FTimespan LocalTime;
						// Parse the local time and check the Mpeg time
						if (ParseTimestamp(LocalTime, Values[LocalIdx][1]) && IsAllNumeric(Values[MpegIdx][1]))
						{
							// The value is given in the typical MPEG-TS 90kHz clock. Convert it into HNS as used by FTimespan.
							FTimespan MpegTS90(FCString::Atoi64(*Values[MpegIdx][1]) * 1000 / 9);

							// The mapping maps a local cue time to some MPEG-TS time.
							// Typically the local time is zero, but could be anything.
							// The offset to be added to all the local times is the difference between the MPEG-TS time and the local cue time.
							NewDoc->TimestampOffset = MpegTS90 - LocalTime;
						}
					}
				}
			}
		}
	}

	// Set the new document as the current one.
	CurrentDoc = MoveTemp(NewDoc);
	return true;
}



void FWebVTTParser::GetCuesAtTime(TArray<TUniquePtr<ICue>>& OutCues, FTimespan& OutNextChangeAt, TUniquePtr<ICueIterator>& InOutIterator, const FTimespan& InAtTime)
{
	if (!CurrentDoc.IsValid() || InAtTime > CurrentDoc->LargestCueEndTime + CurrentDoc->TimestampOffset)
	{
		InOutIterator.Reset();
		OutNextChangeAt = FTimespan::MaxValue();
		return;
	}
	else if (InAtTime < CurrentDoc->SmallestCueStartTime + CurrentDoc->TimestampOffset)
	{
		InOutIterator.Reset();
		OutNextChangeAt = CurrentDoc->SmallestCueStartTime + CurrentDoc->TimestampOffset;
		return;
	}

	// If the iterator is not set up yet, locate the first cue.
	if (!InOutIterator.IsValid())
	{
		InOutIterator = MakeUnique<FPublicCueIterator>();
	}
	FPublicCueIterator* It = static_cast<FPublicCueIterator*>(InOutIterator.Get());
	if (InAtTime < It->CurrentTime)
	{
		It->Reset();
	}
	if (It->CurrentTime == FTimespan::MinValue())
	{
		for(int32 i=0,iMax=CurrentDoc->Cues.Num(); i<iMax; ++i)
		{
			TRange<FTimespan> CueRange(CurrentDoc->Cues[i]->Start + CurrentDoc->TimestampOffset, CurrentDoc->Cues[i]->End + CurrentDoc->TimestampOffset);
			if (CueRange.Contains(InAtTime))
			{
				It->CurrentTime = CueRange.GetLowerBoundValue() + CurrentDoc->TimestampOffset;
				It->CurrentIndex = i;
				break;
			}
		}
		check(It->CurrentIndex >= 0);
		if (It->CurrentIndex < 0)
		{
			return;
		}
	}
	// Look at the next cues starting at the iterator's position.
	TArray<FTimespan> CueTimes;
	for(int32 i=It->CurrentIndex,iMax=CurrentDoc->Cues.Num(); i<iMax; ++i)
	{
		TRange<FTimespan> CueRange(CurrentDoc->Cues[i]->Start + CurrentDoc->TimestampOffset, CurrentDoc->Cues[i]->End + CurrentDoc->TimestampOffset);
		if (CueRange.Contains(InAtTime))
		{
			TUniquePtr<ICue> Cue(new FPublicCue(CurrentDoc->Cues[i], CurrentDoc->TimestampOffset));
			bool bIsDuplicate = false;
			for(int nd=0, ndMax=OutCues.Num(); nd<ndMax; ++nd)
			{
				if (*static_cast<const FPublicCue*>(OutCues[nd].Get()) == *static_cast<const FPublicCue*>(Cue.Get()))
				{
					bIsDuplicate = true;
					break;
				}
			}
			if (!bIsDuplicate)
			{
				OutCues.AddUnique(MoveTemp(Cue));
			}
			// Add the cue range start and end to the cue times for determining the next
			// time we need to get handled.
			CueTimes.Add(CueRange.GetLowerBoundValue());
			CueTimes.Add(CueRange.GetUpperBoundValue());
		}
		else if (CueRange.GetLowerBoundValue() > InAtTime)
		{
			CueTimes.Add(CueRange.GetLowerBoundValue());
			break;
		}
	}
	// Sort the cue times and remove all those prior to the current time.
	CueTimes.Sort();
	while(!CueTimes.IsEmpty() && CueTimes[0] <= InAtTime)
	{
		CueTimes.RemoveAt(0);
	}
	// Then pick the next future time as the one we need to get handled again.
	OutNextChangeAt = CueTimes.Num() ? CueTimes[0] : FTimespan::MaxValue();

	// Skip the iterator ahead over all cues that are now in the past.
	while(It->CurrentIndex < CurrentDoc->Cues.Num() && CurrentDoc->Cues[It->CurrentIndex]->End + CurrentDoc->TimestampOffset < InAtTime)
	{
		++It->CurrentIndex;
		It->CurrentTime = (It->CurrentIndex < CurrentDoc->Cues.Num() ? CurrentDoc->Cues[It->CurrentIndex]->Start : CurrentDoc->LargestCueEndTime) + CurrentDoc->TimestampOffset;
	}
}


} // namespace ElectraWebVTTParser
