// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace Electra { class FParamDict; }
namespace Electra { class FTimeValue; }

namespace ElectraWebVTTParser
{

class IWebVTTParser
{
public:
	static TSharedPtr<IWebVTTParser, ESPMode::ThreadSafe> Create();

	virtual ~IWebVTTParser() = default;

	/**
	 * Returns the most recent error message.
	 */
	virtual const FString& GetLastErrorMessage() const = 0;

	/**
	 * Parses the provided WebVTT document.
	 *
	 * @return true if successful, false if not. Get error message from GetLastErrorMessage().
	 */
	virtual bool ParseWebVTT(const TArray<uint8>& InWebVTTData, const Electra::FParamDict& InOptions) = 0;


	class ICue
	{
	public:
		virtual ~ICue() = default;
		virtual FTimespan GetStartTime() const = 0;
		virtual FTimespan GetEndTime() const = 0;
		virtual FString GetID() const = 0;
		virtual FString GetText() const = 0;
	};

	class ICueIterator
	{
	public:
		virtual ~ICueIterator() = default;
	};

	virtual void GetCuesAtTime(TArray<TUniquePtr<ICue>>& OutCues, FTimespan& OutNextChangeAt, TUniquePtr<ICueIterator>& InOutIterator, const FTimespan& InAtTime) = 0;


	enum class EWebVTTType
	{
		Subtitle,
		Chapters,
		Metadata
	};
	static FString ProcessCueTextForType(FStringView InText, EWebVTTType InType);
};

}
