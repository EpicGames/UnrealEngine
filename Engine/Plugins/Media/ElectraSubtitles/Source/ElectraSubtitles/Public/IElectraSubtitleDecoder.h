// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaSubtitleDecoderOutput.h"
#include "Delegates/Delegate.h"

namespace Electra { class FParamDict; }
namespace Electra { class FTimeValue; }


/**
 * Base class of a subtitle decoder.
 */
class IElectraSubtitleDecoder
{
public:
	virtual ~IElectraSubtitleDecoder() = default;

	//-------------------------------------------------------------------------
	// Initialization methods
	//
	//! Initializes with codec specific data and additional attributes.
	virtual bool InitializeStreamWithCSD(const TArray<uint8>& InCSD, const Electra::FParamDict& InAdditionalInfo) = 0;

	//-------------------------------------------------------------------------
	// Delegate to which the application can subscribe to receive parsed subtitles.
	//
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSubtitleReceivedDelegate, ISubtitleDecoderOutputPtr /* DecodedSubtitle */);
	virtual FOnSubtitleReceivedDelegate& GetParsedSubtitleReceiveDelegate() = 0;

	//-------------------------------------------------------------------------
	// Data handling methods
	//
	//! Returns a time offset at which streamed subtitles should be delivered ahead of their actual timestamp (if they are available earlier)
	virtual Electra::FTimeValue GetStreamedDeliveryTimeOffset() = 0;
	//! Adds streamed subtitle data.
	virtual void AddStreamedSubtitleData(const TArray<uint8>& InData, Electra::FTimeValue InAbsoluteTimestamp, Electra::FTimeValue InDuration, const Electra::FParamDict& InAdditionalInfo) = 0;
	//! Signals no more streamed subtitles will be arriving.
	virtual void SignalStreamedSubtitleEOD() = 0;
	//! Flushes all input and output.
	virtual void Flush() = 0;


	//-------------------------------------------------------------------------
	// Handling methods
	//
	virtual void Start() = 0;
	virtual void Stop() = 0;
	virtual void UpdatePlaybackPosition(Electra::FTimeValue InAbsolutePosition, Electra::FTimeValue InLocalPosition) = 0;

};
