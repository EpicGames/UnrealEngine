// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleInterface.h"


class IElectraSubtitleDecoder;


class IElectraSubtitleModularFeature : public IModularFeature
{
public:
	virtual ~IElectraSubtitleModularFeature() = default;

	/**
	 * Queries whether or not this plugin's modular feature is capable of handling subtitles of
	 * the specified format. The codec string is a commonly known format like "wvtt", "tx3g", "imsc1", etc.
	 * but may also be a MIME type like "application/ttml+xml" for sideloaded subtitles.
	 */
	virtual bool SupportsFormat(const FString& SubtitleCodecName) const = 0;

	/**
	 * Returns a list of the supported subtitle codecs.
	 */
	virtual void GetSupportedFormats(TArray<FString>& OutSupportedCodecNames) const = 0;

	/**
	 * Returns a priority for this plugin's subtitle format feature.
	 * The return value is not standardized, but the decoder returning the largest value for a format
	 * will be used to handle the subtitles.
	 * This is mainly used if you provide a plugin of your own that handles a format better than another
	 * plugin does and you want to make sure yours will be used.
	 */
	virtual int32 GetPriorityForFormat(const FString& SubtitleCodecName) const = 0;

	/**
	 * Called to create a decoder for the given format from this plugin's modular feature.
	 */
	virtual TSharedPtr<IElectraSubtitleDecoder, ESPMode::ThreadSafe> CreateDecoderForFormat(const FString& SubtitleCodecName) = 0;
};



/**
 * Interface for the `ElectraSubtitles` module.
 */
class IElectraSubtitlesModule : public IModuleInterface
{
public:
	virtual ~IElectraSubtitlesModule() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("ElectraSubtitles"));
		return FeatureName;
	}


#if 0
	/**
	 * Singleton-like access to IElectraSubtitlesModule
	 *
	 * @return Returns IElectraSubtitlesModule singleton instance, loading the module on demand if needed
	 */
	static inline IElectraSubtitlesModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IElectraSubtitlesModule>("ElectraSubtitles");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("ElectraSubtitles");
	}
#endif
};
